// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionInteraction.h"
#include "CollisionQueryParams.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

#if WITH_EDITOR
#	include "Editor.h"
#endif


UXRCreativeSelectionInteraction::UXRCreativeSelectionInteraction()
 : TraceCallback([this](const FInputDeviceRay& InRay) { return DefaultTrace(InRay); })
{
}

void UXRCreativeSelectionInteraction::Initialize(
	UTypedElementSelectionSet* InSelectionSet,
	FActorPredicate InCanSelectCallback,
	FTraceMethod InTraceCallback)
{
	ensure(InSelectionSet);
	WeakSelectionSet = InSelectionSet;

	if (InCanSelectCallback)
	{
		CanSelectCallback = MoveTemp(InCanSelectCallback);
	}

	if (InTraceCallback)
	{
		TraceCallback = MoveTemp(InTraceCallback);
	}

	// create click behavior and set ourselves as click target
	ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Modifiers.RegisterModifier(AddToSelectionModifier, FInputDeviceState::IsShiftKeyDown);
	ClickBehavior->Modifiers.RegisterModifier(ToggleSelectionModifier, FInputDeviceState::IsCtrlKeyDown);
	ClickBehavior->Initialize(this);

	BehaviorSet = NewObject<UInputBehaviorSet>();
	BehaviorSet->Add(ClickBehavior, this);
}


void UXRCreativeSelectionInteraction::Shutdown()
{
	for (const AActor* OwnedElementActor : OwnedElementActors)
	{
		TTypedElementOwner<FActorElementData> Element = ActorElementOwnerStore.UnregisterElementOwner(OwnedElementActor);
		UEngineElementsLibrary::DestroyActorElement(OwnedElementActor, Element);
	}

	OwnedElementActors.Empty();
}


void UXRCreativeSelectionInteraction::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	// update modifier state flags
	if (ModifierID == AddToSelectionModifier)
	{
		bAddToSelectionEnabled = bIsOn;
	}
	else if (ModifierID == ToggleSelectionModifier)
	{
		bToggleSelectionEnabled = bIsOn;
	}
}


FInputRayHit UXRCreativeSelectionInteraction::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit RayHit;

	const FHitResult HitResult = TraceCallback(ClickPos);
	AActor* HitActor = HitResult.GetActor();
	if (HitResult.bBlockingHit && CanSelectCallback(HitActor))
	{
		RayHit.bHit = true;
		RayHit.HitDepth = HitResult.Distance;
		RayHit.bHasHitNormal = true;
		RayHit.SetHitObject(HitActor);
	}

	return RayHit;
}


FTypedElementHandle UXRCreativeSelectionInteraction::AcquireActorElementHandle(const AActor* Actor, const bool bAllowCreate)
{
	TTypedElementOwnerScopedAccess<FActorElementData> ExistingElement = ActorElementOwnerStore.FindElementOwner(Actor);
	if (ExistingElement)
	{
		return ExistingElement->AcquireHandle();
	}

	if (bAllowCreate)
	{
		TTypedElementOwnerScopedAccess<FActorElementData> NewElement = ActorElementOwnerStore.RegisterElementOwner(Actor, UEngineElementsLibrary::CreateActorElement(Actor));
		if (NewElement)
		{
			OwnedElementActors.Add(Actor);
			return NewElement->AcquireHandle();
		}
	}

	return FTypedElementHandle();
}


FHitResult UXRCreativeSelectionInteraction::DefaultTrace(const FInputDeviceRay& InRay) const
{
	FCollisionObjectQueryParams QueryParams(FCollisionObjectQueryParams::AllObjects);
	FHitResult Result;
	GetWorld()->LineTraceSingleByObjectType(Result, InRay.WorldRay.Origin, InRay.WorldRay.PointAt(RayLength), QueryParams);
	return Result;
}


void UXRCreativeSelectionInteraction::OnClicked(const FInputDeviceRay& ClickPos)
{
	UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get();
	if (!ensure(SelectionSet))
	{
		return;
	}

	FCollisionObjectQueryParams QueryParams(FCollisionObjectQueryParams::AllObjects);
	const FHitResult Result = TraceCallback(ClickPos);
	if (ensure(Result.bBlockingHit))
	{
		ensure(CanSelectCallback(Result.GetActor()));

		FTypedElementHandle HitActorHandle;

#if WITH_EDITOR
		if (GEditor && !GEditor->PlayWorld)
		{
			HitActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Result.GetActor());
		}
		else
#endif
		{
			HitActorHandle = AcquireActorElementHandle(Result.GetActor(), true);
		}

		// TODO: Handle bAddToSelectionEnabled, bToggleSelectionEnabled
		SelectionSet->ClearSelection(FTypedElementSelectionOptions());
		SelectionSet->SelectElement(HitActorHandle, FTypedElementSelectionOptions());
	}
	else
	{
		SelectionSet->ClearSelection(FTypedElementSelectionOptions());
	}
}
