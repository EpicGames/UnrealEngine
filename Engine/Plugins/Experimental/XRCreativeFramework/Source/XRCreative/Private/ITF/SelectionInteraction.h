// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementOwnerStore.h"
#include "InputBehaviorSet.h"
#include "SelectionInteraction.generated.h"


struct FHitResult;
class UTypedElementSelectionSet;


UCLASS()
class UXRCreativeSelectionInteraction : public UObject, public IInputBehaviorSource, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:
	static constexpr double RayLength = 999999.0;

	using FActorPredicate = TUniqueFunction<bool(AActor*)>;
	using FTraceMethod = TUniqueFunction<FHitResult(const FInputDeviceRay&)>;
	
	UXRCreativeSelectionInteraction();

	/**
	 * Set up the Interaction, creates and registers Behaviors/etc. 
	 * 
	 * @param InSelectionSet the typed element selection set we maintain a weak pointer to and operate on
	 * @param InCanSelectCallback this function will be called to determine if the selection can be changed to the specified actor (or null); when a tool is active, we may wish to lock selection
	 * @param InTraceCallback can be provided to override the default scene trace implementation
	 */
	void Initialize(
		UTypedElementSelectionSet* InSelectionSet,
		FActorPredicate InCanSelectCallback = nullptr,
		FTraceMethod InTraceCallback = nullptr);

	void Shutdown();

	// IInputBehaviorSource interface
	virtual const UInputBehaviorSet* GetInputBehaviors() const { return BehaviorSet; }

	// IClickBehaviorTarget interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IModifierToggleBehaviorTarget interface
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

protected:
	FTypedElementHandle AcquireActorElementHandle(const AActor* Actor, const bool bAllowCreate);

	FHitResult DefaultTrace(const FInputDeviceRay& InRay) const;

protected:
	// click-to-select behavior
	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickBehavior;

	// set of all behaviors, will be passed up to UInputRouter
	UPROPERTY()
	TObjectPtr<UInputBehaviorSet> BehaviorSet;

	// TODO: This should either be in a subsystem, or better yet the existing editor-only stores should be made not-editor-only.
	TSet<const AActor*> OwnedElementActors;
	TTypedElementOwnerStore<FActorElementData, const AActor*> ActorElementOwnerStore;

	TWeakObjectPtr<UTypedElementSelectionSet> WeakSelectionSet;

	// default predicate allows anything
	FActorPredicate CanSelectCallback = [](AActor*) { return true; };

	FTraceMethod TraceCallback;

	// flags used to identify behavior modifier keys/buttons
	static const int AddToSelectionModifier = 1;
	bool bAddToSelectionEnabled = false;

	static const int ToggleSelectionModifier = 2;
	bool bToggleSelectionEnabled = false;
};
