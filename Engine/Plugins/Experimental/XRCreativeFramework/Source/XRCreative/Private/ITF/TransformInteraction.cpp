// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformInteraction.h"
#include "XRCreativeGizmos.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/GizmoBaseComponent.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/TransformProxy.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "ContextObjectStore.h"
#include "Components/PrimitiveComponent.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "InteractiveGizmoManager.h"


const FString UXRCreativeTransformInteraction::GizmoBuilderIdentifier("XRCreativeGizmo");


void UXRCreativeTransformInteraction::Initialize(
	TSharedRef<FCombinedTransformGizmoActorFactory> InGizmoActorFactory,
	UTypedElementSelectionSet* InSelectionSet,
	UInteractiveGizmoManager* InGizmoManager,
	TUniqueFunction<bool()> InGizmoEnabledCallback)
{
	check(InSelectionSet && IsValid(InSelectionSet));
	check(InGizmoManager && IsValid(InGizmoManager));

	GizmoActorFactory = InGizmoActorFactory;
	WeakSelectionSet = InSelectionSet;
	WeakGizmoManager = InGizmoManager;
	GizmoEnabledCallback = MoveTemp(InGizmoEnabledCallback);

	UXRCreativeGizmoBuilder* GizmoBuilder = NewObject<UXRCreativeGizmoBuilder>();
	GizmoBuilder->GizmoActorBuilder = GizmoActorFactory;
	InGizmoManager->RegisterGizmoType(GizmoBuilderIdentifier, GizmoBuilder);

	SelectionChangedEventHandle = InSelectionSet->OnChanged().AddWeakLambda(this,
		[this](const UTypedElementSelectionSet* InSelectionSet)
		{
			UpdateGizmoTargets(InSelectionSet);
		}
	);
}


void UXRCreativeTransformInteraction::Shutdown()
{
	if (SelectionChangedEventHandle.IsValid())
	{
		if (UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get())
		{
			SelectionSet->OnChanged().Remove(SelectionChangedEventHandle);
		}

		SelectionChangedEventHandle.Reset();
	}

	if (WeakGizmoManager.IsValid())
	{
		UpdateGizmoTargets(nullptr);
	}
}


void UXRCreativeTransformInteraction::SetEnableScaling(bool bEnable)
{
	if (bEnable != bEnableScaling)
	{
		bEnableScaling = bEnable;
		ForceUpdateGizmoState();
	}
}


void UXRCreativeTransformInteraction::SetEnableNonUniformScaling(bool bEnable)
{
	if (bEnable != bEnableNonUniformScaling)
	{
		bEnableNonUniformScaling = bEnable;
		ForceUpdateGizmoState();
	}
}


void UXRCreativeTransformInteraction::ForceUpdateGizmoState()
{
	UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get();
	ensure(SelectionSet);
	UpdateGizmoTargets(SelectionSet);
}


void UXRCreativeTransformInteraction::UpdateGizmoTargets(const UTypedElementSelectionSet* InSelectionSet)
{
	UInteractiveGizmoManager* GizmoManager = WeakGizmoManager.Get();
	if (!ensure(GizmoManager))
	{
		return;
	}

	// destroy existing gizmos if we have any
	if (TransformGizmo != nullptr)
	{
		if (IsValid(TransformProxy))
		{
			TransformProxy->OnEndTransformEdit.Remove(EndTransformEditHandle);
			TransformProxy->OnTransformChangedUndoRedo.Remove(TransformChangedUndoRedoHandle);
		}
		EndTransformEditHandle.Reset();
		TransformChangedUndoRedoHandle.Reset();
		ManipulatedActors.Reset();

		GizmoManager->DestroyAllGizmosByOwner(this);
		TransformGizmo = nullptr;
		TransformProxy = nullptr;
	}

	// if no selection, no gizmo
	if (!InSelectionSet || GizmoEnabledCallback() == false)
	{
		return;
	}

	TArray<AActor*> Selection = InSelectionSet->GetSelectedObjects<AActor>();
	if (Selection.Num() == 0)
	{
		return;
	}

	TransformProxy = NewObject<UTransformProxy>(this);

	ManipulatedActors.Reserve(Selection.Num());
	for (AActor* Actor : Selection)
	{
		if (Actor && Actor->GetRootComponent())
		{
			TransformProxy->AddComponent(Actor->GetRootComponent());
			ManipulatedActors.Add(Actor);
		}
	}

	// Hook end-of-drag and undo/redo so we can re-publish each moved actor's primitive
	// components into the editor scene-query acceleration structure. Without this, the
	// editor-world chaos scene doesn't refresh moved bodies between physics ticks (which
	// never happen in the editor world), and our ECC_Visibility pointer traces miss the
	// actor at its new pose.
	EndTransformEditHandle = TransformProxy->OnEndTransformEdit.AddWeakLambda(this,
		[this](UTransformProxy*) { NotifyActorsMoved(/*bFinished*/ true); });

	TransformChangedUndoRedoHandle = TransformProxy->OnTransformChangedUndoRedo.AddWeakLambda(this,
		[this](UTransformProxy*, FTransform) { NotifyActorsMoved(/*bFinished*/ true); });

	ETransformGizmoSubElements GizmoElements = ETransformGizmoSubElements::FullTranslateRotateScale;
	if (bEnableScaling == false)
	{
		GizmoElements = ETransformGizmoSubElements::StandardTranslateRotate;
	}
	else if (bEnableNonUniformScaling == false || Selection.Num() > 1)
	{
		// cannot non-uniform scale multiple objects
		GizmoElements = ETransformGizmoSubElements::TranslateRotateUniformScale;
	}

	//TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, GizmoElements, this);
	GizmoActorFactory->EnableElements = GizmoElements;
	TransformGizmo = CastChecked<UCombinedTransformGizmo>(GizmoManager->CreateGizmo(GizmoBuilderIdentifier, FString(), this));
	TransformGizmo->SetActiveTarget(TransformProxy);

	AXRCreativeCombinedTransformGizmoActor* NewGizmoActor =
		CastChecked<AXRCreativeCombinedTransformGizmoActor>(TransformGizmo->GetGizmoActor());
	NewGizmoActor->WeakGizmoManager = GizmoManager;

	// optionally ignore coordinate system setting
	//TransformGizmo->bUseContextCoordinateSystem = false;
	//TransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
}


void UXRCreativeTransformInteraction::NotifyActorsMoved(bool bFinished)
{
#if WITH_EDITOR
	if (!bFinished)
	{
		return;
	}

	for (const TWeakObjectPtr<AActor>& WeakActor : ManipulatedActors)
	{
		AActor* Actor = WeakActor.Get();
		if (!Actor)
		{
			continue;
		}

		// Re-publish physics state for each moved primitive. The editor world doesn't
		// tick its chaos solver, so a SetGlobalPose-only update from SetWorldTransform
		// is not picked up by subsequent LineTraceSingleByChannel calls. Destroying
		// and re-creating the body forces a fresh registration into the scene-query
		// acceleration structure at the current transform.
		for (UActorComponent* Comp : Actor->GetComponents())
		{
			if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Comp))
			{
				if (PrimComp->IsRegistered() && PrimComp->IsPhysicsStateCreated())
				{
					PrimComp->RecreatePhysicsState();
				}
			}
		}

		// Match the standard editor's post-move bookkeeping (lighting cache reset,
		// OnActorMoved broadcast, navigation update, RootComponent->PostEditComponentMove).
		Actor->PostEditMove(/*bFinished*/ true);
	}
#endif
}
