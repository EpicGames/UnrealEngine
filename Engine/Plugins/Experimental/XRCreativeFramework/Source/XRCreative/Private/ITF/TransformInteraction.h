// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TransformInteraction.generated.h"


class AActor;
class FCombinedTransformGizmoActorFactory;
class FXRCreativeTransformGizmoActorFactory;
class UTransformProxy;
class UCombinedTransformGizmo;
class UInteractiveGizmoManager;
class UTypedElementSelectionSet;


/**
 * UXRCreativeTransformInteraction manages a 3D Translate/Rotate/Scale (TRS) Gizmo.
 *
 * Gizmo local/global frame is not controlled here, the Gizmo looks this information up itself
 * based on the EToolContextCoordinateSystem provided by the IToolsContextQueriesAPI implementation.
 * You can configure the Gizmo to ignore this, in UpdateGizmoTargets()
 *
 * Behavior of the TRS Gizmo (ie pivot position, etc) is controlled by a standard UTransformProxy.
 */
UCLASS()
class UXRCreativeTransformInteraction : public UObject
{
	GENERATED_BODY()

	static const FString GizmoBuilderIdentifier;

public:
	/**
	 * Set up the transform interaction. 
	 * @param InGizmoEnabledCallback callback that determines if Gizmo should be created and visible. For example during a Tool we generally want to hide the TRS Gizmo.
	 */
	void Initialize(TSharedRef<FCombinedTransformGizmoActorFactory> InGizmoActorFactory,
		UTypedElementSelectionSet* InSelectionSet,
		UInteractiveGizmoManager* InGizmoManager,
		TUniqueFunction<bool()> InGizmoEnabledCallback);

	void Shutdown();

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void SetEnableScaling(bool bEnable);

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void SetEnableNonUniformScaling(bool bEnable);

	// Recreate Gizmo. Call when external state changes, like set of selected objects
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void ForceUpdateGizmoState();

protected:
	FDelegateHandle SelectionChangedEventHandle;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo;

	TWeakObjectPtr<UTypedElementSelectionSet> WeakSelectionSet;
	TWeakObjectPtr<UInteractiveGizmoManager> WeakGizmoManager;

	TSharedPtr<FCombinedTransformGizmoActorFactory> GizmoActorFactory;

	// Actors driven by TransformProxy. Notified via PostEditMove so the editor world's
	// scene-query acceleration structure (used by ECC_Visibility line traces) stays in
	// sync with their moved collision after a gizmo drag.
	TArray<TWeakObjectPtr<AActor>> ManipulatedActors;

	FDelegateHandle EndTransformEditHandle;
	FDelegateHandle TransformChangedUndoRedoHandle;

	void UpdateGizmoTargets(const UTypedElementSelectionSet* InSelectionSet);

	void NotifyActorsMoved(bool bFinished);

	bool bEnableScaling = true;
	bool bEnableNonUniformScaling = true;

	TUniqueFunction<bool()> GizmoEnabledCallback = [&]() { return true; };
};
