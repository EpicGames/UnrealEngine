// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeGizmos.h"
#include "BaseGizmos/AxisAngleGizmo.h"
#include "BaseGizmos/AxisPositionGizmo.h"
#include "BaseGizmos/GizmoArrowComponent.h"
#include "BaseGizmos/GizmoBoxComponent.h"
#include "BaseGizmos/GizmoCircleComponent.h"
#include "BaseGizmos/GizmoRectangleComponent.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/PlanePositionGizmo.h"
#include "BaseGizmos/TransformSubGizmoUtil.h"
#include "ContextObjectStore.h"
#include "Materials/MaterialInstanceDynamic.h"


UXRCreativeGizmoBuilder::UXRCreativeGizmoBuilder()
{
	AxisPositionBuilderIdentifier = UInteractiveGizmoManager::DefaultAxisPositionBuilderIdentifier;
	PlanePositionBuilderIdentifier = UInteractiveGizmoManager::DefaultPlanePositionBuilderIdentifier;
	AxisAngleBuilderIdentifier = UInteractiveGizmoManager::DefaultAxisAngleBuilderIdentifier;
}

UInteractiveGizmo* UXRCreativeGizmoBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	// FIXME: Have to duplicate base implementation to customize instantiated gizmo type
	// Substituting UXRCreativeGizmo on the next line is the only difference from the base
	UCombinedTransformGizmo* NewGizmo = NewObject<UXRCreativeGizmo>(SceneState.GizmoManager);
	NewGizmo->SetWorld(SceneState.World);

	UGizmoViewContext* GizmoViewContext = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();
	check(IsValid(GizmoViewContext));

	// use default gizmo actor if client has not given us a new builder
	NewGizmo->SetGizmoActorBuilder(GizmoActorBuilder ? GizmoActorBuilder : MakeShared<FCombinedTransformGizmoActorFactory>(GizmoViewContext));

	NewGizmo->SetSubGizmoBuilderIdentifiers(AxisPositionBuilderIdentifier, PlanePositionBuilderIdentifier, AxisAngleBuilderIdentifier);

	// override default hover function if proposed
	if (UpdateHoverFunction)
	{
		NewGizmo->SetUpdateHoverFunction(UpdateHoverFunction);
	}

	if (UpdateCoordSystemFunction)
	{
		NewGizmo->SetUpdateCoordSystemFunction(UpdateCoordSystemFunction);
	}

	return NewGizmo;
}


//////////////////////////////////////////////////////////////////////////


void UXRCreativeGizmo::Setup()
{
	Super::Setup();

	// FIXME: Most ITF sub-gizmos seem to be missing calls to update this
	UpdateInteractingFunction = [](UPrimitiveComponent* Component, bool bInteracting)
	{
		if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Component))
		{
			XrMeshComp->UpdateInteractingState(bInteracting);
		}
	};

	//////////////////////////////////////////////////////////////////////////
	// FIXME?: Have to set these here after base class would otherwise overwrite them

	UpdateHoverFunction = [](UPrimitiveComponent* Component, bool bHovering)
	{
		if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Component))
		{
			XrMeshComp->UpdateHoverState(bHovering);
		}
	};

	UpdateCoordSystemFunction = [this](UPrimitiveComponent* Component, EToolContextCoordinateSystem CoordSystem)
	{
		const bool bWorld = CoordSystem == EToolContextCoordinateSystem::World;
		if (UGizmoBaseComponent* GizmoBase = Cast<UGizmoBaseComponent>(Component))
		{
			GizmoBase->UpdateWorldLocalState(bWorld);
		}
		else if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Component))
		{
			FViewCameraState CameraState;
			GetGizmoManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

			const FTransform Transform = XrMeshComp->CalcViewDependent(CameraState, CoordSystem);
			XrMeshComp->SetRelativeTransform(Transform);
		}
	};
}


void UXRCreativeGizmo::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	if (ACombinedTransformGizmoActor* Actor = GetGizmoActor())
	{
		UInteractiveGizmoManager* GizmoManager = GetGizmoManager();

		FViewCameraState CameraState;
		GizmoManager->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

		const double GizmoDist = (Actor->GetActorLocation() - CameraState.Position).Length();

		const double ConstantScreenSizeBeyondDist = 300.0;
		const double ScaleFactor = FMath::Max(ConstantScreenSizeBeyondDist, GizmoDist) / ConstantScreenSizeBeyondDist;
		Actor->GetRootComponent()->SetRelativeScale3D(FVector(ScaleFactor));

		const bool bLocal = GizmoManager->GetContextQueriesAPI()->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::Local;
		if (!bLocal)
		{
			Actor->GetRootComponent()->SetWorldRotation(FQuat::Identity);
		}
	}
}


UInteractiveGizmo* UXRCreativeGizmo::AddAxisTranslationGizmo(
	FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState)
{
	UAxisPositionGizmo* Gizmo = Cast<UAxisPositionGizmo>(Super::AddAxisTranslationGizmo(Params, SharedState));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	if (UpdateInteractingFunction)
	{
		if (UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(Gizmo->HitTarget.GetObject()))
		{
			HitTarget->UpdateInteractingFunction = [Component = Params.Component, this](bool bInteracting)
			{
				this->UpdateInteractingFunction(Component, bInteracting);
			};

			HitTarget->UpdateHoverFunction = [Component = Params.Component, this](bool bHovering)
			{
				this->UpdateHoverFunction(Component, bHovering);
			};
		}
	}

	if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Params.Component))
	{
		XrMeshComp->AxisSource = Gizmo->AxisSource.GetInterface();
	}

	return Gizmo;
}

UInteractiveGizmo* UXRCreativeGizmo::AddPlaneTranslationGizmo(
	FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState)
{
	UPlanePositionGizmo* Gizmo = Cast<UPlanePositionGizmo>(Super::AddPlaneTranslationGizmo(Params, SharedState));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	if (UpdateInteractingFunction)
	{
		if (UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(Gizmo->HitTarget.GetObject()))
		{
			HitTarget->UpdateInteractingFunction = [Component = Params.Component, this](bool bInteracting)
			{
				this->UpdateInteractingFunction(Component, bInteracting);
			};

			HitTarget->UpdateHoverFunction = [Component = Params.Component, this](bool bHovering)
			{
				this->UpdateHoverFunction(Component, bHovering);
			};
		}
	}

	if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Params.Component))
	{
		XrMeshComp->AxisSource = Gizmo->AxisSource.GetInterface();
	}

	return Gizmo;
}

UInteractiveGizmo* UXRCreativeGizmo::AddAxisRotationGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UAxisAngleGizmo* Gizmo = Cast<UAxisAngleGizmo>(Super::AddAxisRotationGizmo(Params, SharedState));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	if (UpdateInteractingFunction)
	{
		if (UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(Gizmo->HitTarget.GetObject()))
		{
			HitTarget->UpdateInteractingFunction = [Component = Params.Component, this](bool bInteracting)
			{
				this->UpdateInteractingFunction(Component, bInteracting);
			};

			HitTarget->UpdateHoverFunction = [Component = Params.Component, this](bool bHovering)
			{
				this->UpdateHoverFunction(Component, bHovering);
			};
		}
	}

	if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Params.Component))
	{
		XrMeshComp->AxisSource = Gizmo->AxisSource.GetInterface();
	}

	return Gizmo;
}

UInteractiveGizmo* UXRCreativeGizmo::AddAxisScaleGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UAxisPositionGizmo* Gizmo = Cast<UAxisPositionGizmo>(Super::AddAxisScaleGizmo(Params, SharedState));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	if (UpdateInteractingFunction)
	{
		if (UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(Gizmo->HitTarget.GetObject()))
		{
			HitTarget->UpdateInteractingFunction = [Component = Params.Component, this](bool bInteracting)
			{
				this->UpdateInteractingFunction(Component, bInteracting);
			};

			HitTarget->UpdateHoverFunction = [Component = Params.Component, this](bool bHovering)
			{
				this->UpdateHoverFunction(Component, bHovering);
			};
		}
	}

	if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Params.Component))
	{
		XrMeshComp->AxisSource = Gizmo->AxisSource.GetInterface();
	}

	return Gizmo;
}

UInteractiveGizmo* UXRCreativeGizmo::AddPlaneScaleGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UPlanePositionGizmo* Gizmo = Cast<UPlanePositionGizmo>(Super::AddPlaneScaleGizmo(Params, SharedState));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	if (UpdateInteractingFunction)
	{
		if (UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(Gizmo->HitTarget.GetObject()))
		{
			HitTarget->UpdateInteractingFunction = [Component = Params.Component, this](bool bInteracting)
			{
				this->UpdateInteractingFunction(Component, bInteracting);
			};

			HitTarget->UpdateHoverFunction = [Component = Params.Component, this](bool bHovering)
			{
				this->UpdateHoverFunction(Component, bHovering);
			};
		}
	}

	if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Params.Component))
	{
		XrMeshComp->AxisSource = Gizmo->AxisSource.GetInterface();
	}

	return Gizmo;
}

UInteractiveGizmo* UXRCreativeGizmo::AddUniformScaleGizmo(
	FTransformSubGizmoCommonParams& Params,	FTransformSubGizmoSharedState& SharedState)
{
	UPlanePositionGizmo* Gizmo = Cast<UPlanePositionGizmo>(Super::AddUniformScaleGizmo(Params, SharedState));
	if (!ensure(Gizmo))
	{
		return nullptr;
	}

	if (UpdateInteractingFunction)
	{
		if (UGizmoComponentHitTarget* HitTarget = Cast<UGizmoComponentHitTarget>(Gizmo->HitTarget.GetObject()))
		{
			HitTarget->UpdateInteractingFunction = [Component = Params.Component, this](bool bInteracting)
			{
				this->UpdateInteractingFunction(Component, bInteracting);
			};

			HitTarget->UpdateHoverFunction = [Component = Params.Component, this](bool bHovering)
			{
				this->UpdateHoverFunction(Component, bHovering);
			};
		}
	}

	if (UXRCreativeGizmoMeshComponent* XrMeshComp = Cast<UXRCreativeGizmoMeshComponent>(Params.Component))
	{
		XrMeshComp->AxisSource = Gizmo->AxisSource.GetInterface();
	}

	return Gizmo;
}


//////////////////////////////////////////////////////////////////////////


void UXRCreativeGizmoMeshComponent::Initialize()
{
	const int32 NumMaterials = GetNumMaterials();
	Materials.Empty(NumMaterials);
	for (int MaterialIdx = 0; MaterialIdx < NumMaterials; ++MaterialIdx)
	{
		UMaterialInterface* BaseMaterial = GetMaterial(MaterialIdx);
		UMaterialInstanceDynamic* Dynamic = UMaterialInstanceDynamic::Create(BaseMaterial, this);
		SetMaterial(MaterialIdx, Dynamic);
		Materials.Add(Dynamic);
	}
}


void UXRCreativeGizmoMeshComponent::UpdateHoverState(bool bNewHovering)
{
	if (bHovering == bNewHovering)
	{
		return;
	}

	bHovering = bNewHovering;

	OnHoveringChanged.Broadcast(this, bNewHovering);

	static const FName HoverMaterialParameterName("GizmoComponentHovering");
	for (UMaterialInstanceDynamic* Material : Materials)
	{
		Material->SetScalarParameterValue(HoverMaterialParameterName, bHovering ? 1.0f : 0.0f);
	}
}


void UXRCreativeGizmoMeshComponent::UpdateInteractingState(bool bNewInteracting)
{
	if (bInteracting == bNewInteracting)
	{
		return;
	}

	bInteracting = bNewInteracting;

	OnInteractingChanged.Broadcast(this, bNewInteracting);

	static const FName InteractingMaterialParameterName("GizmoComponentInteracting");
	for (UMaterialInstanceDynamic* Material : Materials)
	{
		Material->SetScalarParameterValue(InteractingMaterialParameterName, bInteracting ? 1.0f : 0.0f);
	}
}


FTransform UXRCreativeGizmoMeshComponent::CalcViewDependent(
	const FViewCameraState& InView,
	EToolContextCoordinateSystem InCoords
) const
{
	check(AxisSource);

	UXRCreativeGizmoMeshComponent* Archetype = CastChecked<UXRCreativeGizmoMeshComponent>(GetArchetype());

	FTransform ResultTransform = Archetype->GetRelativeTransform();

	FVector WorldAxisDirection, WorldAxisTangentX, WorldAxisTangentY;
	FVector CompLocalAxisDirection, CompLocalAxisTangentX, CompLocalAxisTangentY;

	const FTransform& GizmoTransform = GetAttachmentRoot()->GetComponentTransform();

	const bool bWorld = InCoords == EToolContextCoordinateSystem::World;
	if (bWorld)
	{
		AxisSource->GetAxisFrame(CompLocalAxisDirection, CompLocalAxisTangentX, CompLocalAxisTangentY);
		WorldAxisDirection = CompLocalAxisDirection;
		WorldAxisTangentX = CompLocalAxisTangentX;
		WorldAxisTangentY = CompLocalAxisTangentY;
	}
	else
	{
		AxisSource->GetAxisFrame(WorldAxisDirection, WorldAxisTangentX, WorldAxisTangentY);
		CompLocalAxisDirection = GizmoTransform.InverseTransformVectorNoScale(WorldAxisDirection);
		CompLocalAxisTangentX = GizmoTransform.InverseTransformVectorNoScale(WorldAxisTangentX);
		CompLocalAxisTangentY = GizmoTransform.InverseTransformVectorNoScale(WorldAxisTangentY);
	}

	const FVector ViewDirection = (GizmoTransform.GetLocation() - InView.Position).GetSafeNormal();
	const float ViewDotAxisDirection = ViewDirection.Dot(WorldAxisDirection);
	const float ViewDotAxisTangentX = ViewDirection.Dot(WorldAxisTangentX);
	const float ViewDotAxisTangentY = ViewDirection.Dot(WorldAxisTangentY);

	if (HideAbsoluteViewDotThreshold > 0.0f && FMath::Abs(ViewDotAxisDirection) <= HideAbsoluteViewDotThreshold)
	{
		ResultTransform.SetScale3D(FVector::ZeroVector);
		return ResultTransform;
	}

	FVector ReflectionScale = FVector::OneVector;

	if (bReflectOnPrimaryAxis && ViewDotAxisDirection > 0.0f)
	{
		ReflectionScale += CompLocalAxisDirection * -2.0;
	}

	if (bReflectOnTangentAxes)
	{
		if (ViewDotAxisTangentX > 0.0f)
		{
			ReflectionScale += CompLocalAxisTangentX * -2.0;
		}

		if (ViewDotAxisTangentY > 0.0f)
		{
			ReflectionScale += CompLocalAxisTangentY * -2.0;
		}
	}

	const FTransform ReflectScale = FTransform(FQuat::Identity, FVector::ZeroVector, ReflectionScale);
	ResultTransform = ResultTransform * ReflectScale;

	return ResultTransform;
}


//////////////////////////////////////////////////////////////////////////


AXRCreativeCombinedTransformGizmoActor::AXRCreativeCombinedTransformGizmoActor()
{
	XRSceneRoot = RootComponent;

	TranslateX = XRTranslateX = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("TranslateX");
	TranslateX->SetupAttachment(GetRootComponent());
	TranslateY = XRTranslateY = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("TranslateY");
	TranslateY->SetupAttachment(GetRootComponent());
	TranslateZ = XRTranslateZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("TranslateZ");
	TranslateZ->SetupAttachment(GetRootComponent());
	TranslateYZ = XRTranslateYZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("TranslateYZ");
	TranslateYZ->SetupAttachment(GetRootComponent());
	TranslateXZ = XRTranslateXZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("TranslateXZ");
	TranslateXZ->SetupAttachment(GetRootComponent());
	TranslateXY = XRTranslateXY = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("TranslateXY");
	TranslateXY->SetupAttachment(GetRootComponent());

	// TODO: RotationSphere?
	RotateX = XRRotateX = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("RotateX");
	RotateX->SetupAttachment(GetRootComponent());
	RotateY = XRRotateY = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("RotateY");
	RotateY->SetupAttachment(GetRootComponent());
	RotateZ = XRRotateZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("RotateZ");
	RotateZ->SetupAttachment(GetRootComponent());

	UniformScale = XRUniformScale = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("UniformScale");
	UniformScale->SetupAttachment(GetRootComponent());
	AxisScaleX = XRAxisScaleX = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("AxisScaleX");
	AxisScaleX->SetupAttachment(GetRootComponent());
	AxisScaleY = XRAxisScaleY = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("AxisScaleY");
	AxisScaleY->SetupAttachment(GetRootComponent());
	AxisScaleZ = XRAxisScaleZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("AxisScaleZ");
	AxisScaleZ->SetupAttachment(GetRootComponent());
	PlaneScaleYZ = XRPlaneScaleYZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("AxisScaleYZ");
	PlaneScaleYZ->SetupAttachment(GetRootComponent());
	PlaneScaleXZ = XRPlaneScaleXZ = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("AxisScaleXZ");
	PlaneScaleXZ->SetupAttachment(GetRootComponent());
	PlaneScaleXY = XRPlaneScaleXY = CreateOptionalDefaultSubobject<UXRCreativeGizmoMeshComponent>("AxisScaleXY");
	PlaneScaleXY->SetupAttachment(GetRootComponent());
}


void AXRCreativeCombinedTransformGizmoActor::SetEnabledElements(ETransformGizmoSubElements EnableElements)
{
	const bool bEnableTranslateAxisX = (EnableElements & ETransformGizmoSubElements::TranslateAxisX) != ETransformGizmoSubElements::None;
	const bool bEnableTranslateAxisY = (EnableElements & ETransformGizmoSubElements::TranslateAxisY) != ETransformGizmoSubElements::None;
	const bool bEnableTranslateAxisZ = (EnableElements & ETransformGizmoSubElements::TranslateAxisZ) != ETransformGizmoSubElements::None;
	if (TranslateX)
	{
		TranslateX->SetVisibility(bEnableTranslateAxisX, true);
	}
	if (TranslateY)
	{
		TranslateY->SetVisibility(bEnableTranslateAxisY, true);
	}
	if (TranslateZ)
	{
		TranslateZ->SetVisibility(bEnableTranslateAxisZ, true);
	}

	const bool bEnableTransplatePlaneYZ = (EnableElements & ETransformGizmoSubElements::TranslatePlaneYZ) != ETransformGizmoSubElements::None;
	const bool bEnableTransplatePlaneXZ = (EnableElements & ETransformGizmoSubElements::TranslatePlaneXZ) != ETransformGizmoSubElements::None;
	const bool bEnableTransplatePlaneXY = (EnableElements & ETransformGizmoSubElements::TranslatePlaneXY) != ETransformGizmoSubElements::None;
	if (TranslateYZ)
	{
		TranslateYZ->SetVisibility(bEnableTransplatePlaneYZ, true);
	}
	if (TranslateXZ)
	{
		TranslateXZ->SetVisibility(bEnableTransplatePlaneXZ, true);
	}
	if (TranslateXY)
	{
		TranslateXY->SetVisibility(bEnableTransplatePlaneXY, true);
	}

	const bool bEnableRotateAxisX = (EnableElements & ETransformGizmoSubElements::RotateAxisX) != ETransformGizmoSubElements::None;
	const bool bEnableRotateAxisY = (EnableElements & ETransformGizmoSubElements::RotateAxisY) != ETransformGizmoSubElements::None;
	const bool bEnableRotateAxisZ = (EnableElements & ETransformGizmoSubElements::RotateAxisZ) != ETransformGizmoSubElements::None;
	if (RotateX)
	{
		RotateX->SetVisibility(bEnableRotateAxisX, true);
	}
	if (RotateY)
	{
		RotateY->SetVisibility(bEnableRotateAxisY, true);
	}
	if (RotateZ)
	{
		RotateZ->SetVisibility(bEnableRotateAxisZ, true);
	}

#if 0
	// add a non-interactive view-aligned circle element, so the axes look like a sphere.
	const bool bAnyRotate = bEnableRotateAxisX || bEnableRotateAxisY || bEnableRotateAxisZ;
	if (bAnyRotate)
	{
		UGizmoCircleComponent* RotationSphereComp = XR	UGizmoCircleComponent* RotationSphereComp = CreateOptionalDefaultSubobject<UGizmoCircleComponent>("RotationSphere");
		RotationSphereComp->SetupAttachment(GetRootComponent());
		RotationSphereComp->Color = FLinearColor::Gray;
		RotationSphereComp->Thickness = GizmoLineThickness * 0.5f;
		RotationSphereComp->Radius = 120.0f;
		RotationSphereComp->bViewAligned = true;
		RotationSphereComp->NotifyExternalPropertyUpdates();
		RotationSphere = RotationSphereComp;
	}
#endif

	const bool bEnableScaleUniform = (EnableElements & ETransformGizmoSubElements::ScaleUniform) != ETransformGizmoSubElements::None;
	const bool bEnableScaleAxisX = (EnableElements & ETransformGizmoSubElements::ScaleAxisX) != ETransformGizmoSubElements::None;
	const bool bEnableScaleAxisY = (EnableElements & ETransformGizmoSubElements::ScaleAxisY) != ETransformGizmoSubElements::None;
	const bool bEnableScaleAxisZ = (EnableElements & ETransformGizmoSubElements::ScaleAxisZ) != ETransformGizmoSubElements::None;
	if (UniformScale)
	{
		UniformScale->SetVisibility(bEnableScaleUniform, true);
	}
	if (AxisScaleX)
	{
		AxisScaleX->SetVisibility(bEnableScaleAxisX, true);
	}
	if (AxisScaleY)
	{
		AxisScaleY->SetVisibility(bEnableScaleAxisY, true);
	}
	if (AxisScaleZ)
	{
		AxisScaleZ->SetVisibility(bEnableScaleAxisZ, true);
	}

	const bool bEnableScalePlaneYZ = (EnableElements & ETransformGizmoSubElements::ScalePlaneYZ) != ETransformGizmoSubElements::None;
	const bool bEnableScalePlaneXZ = (EnableElements & ETransformGizmoSubElements::ScalePlaneXZ) != ETransformGizmoSubElements::None;
	const bool bEnableScalePlaneXY = (EnableElements & ETransformGizmoSubElements::ScalePlaneXY) != ETransformGizmoSubElements::None;
	if (PlaneScaleYZ)
	{
		PlaneScaleYZ->SetVisibility(bEnableScalePlaneYZ, true);
	}
	if (PlaneScaleXZ)
	{
		PlaneScaleXZ->SetVisibility(bEnableScalePlaneXZ, true);
	}
	if (PlaneScaleXY)
	{
		PlaneScaleXY->SetVisibility(bEnableScalePlaneXY, true);
	}
}
