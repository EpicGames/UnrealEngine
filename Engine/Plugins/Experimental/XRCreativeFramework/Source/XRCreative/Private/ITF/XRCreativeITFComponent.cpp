// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeITFComponent.h"
#include "XRCreativeAvatar.h"
#include "XRCreativeITFRenderComponent.h"
#include "XRCreativeGizmos.h"
#include "XRCreativeLog.h"
#include "XRCreativePointerComponent.h"
#include "SelectionInteraction.h"
#include "TransformInteraction.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Components/PrimitiveComponent.h"
#include "ContextObjectStore.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "InteractiveToolsContext.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MotionControllerComponent.h"
#include "SceneView.h"
#include "Slate/SceneViewport.h"
#include "Slate/SGameLayerManager.h"

#if WITH_EDITOR
#	include "Editor.h"
#	include "Editor/TransBuffer.h"
#	include "Editor/UnrealEdEngine.h"
#	include "IVREditorModule.h"
#	include "LevelEditorSubsystem.h"
#	include "ScopedTransaction.h"
#	include "SLevelViewport.h"
#	include "UnrealEdGlobals.h"
#	include "VREditorModeBase.h"
#endif


class FXRCreativeToolsContextQueriesImpl : public IToolsContextQueriesAPI
{
public:
	FXRCreativeToolsContextQueriesImpl(UInteractiveToolsContext* InContext, UXRCreativeITFComponent* InToolsComp)
		: ToolsContext(InContext)
		, ToolsComp(InToolsComp)
		, ContextActor(Cast<APawn>(InToolsComp->GetOwner()))
	{
	}

	void UpdateActiveViewport(FViewport* Viewport)
	{
		ActiveViewport = Viewport;
	}

	virtual UWorld* GetCurrentEditingWorld() const override
	{
		return ToolsContext->GetWorld();
	}

	virtual void GetCurrentSelectionState(FToolBuilderState& StateOut) const override
	{
		StateOut.ToolManager = ToolsContext->ToolManager;
		StateOut.TargetManager = ToolsContext->TargetManager;
		StateOut.GizmoManager = ToolsContext->GizmoManager;
		StateOut.World = ToolsContext->GetWorld();

		UTypedElementSelectionSet* SelectionSet = ToolsComp->GetSelectionSet();
		StateOut.SelectedActors.Append(SelectionSet->GetSelectedObjects<AActor>());
		StateOut.SelectedComponents.Append(SelectionSet->GetSelectedObjects<USceneComponent>());
	}

	virtual void GetCurrentViewState(FViewCameraState& StateOut) const override
	{
		if (!ContextActor)
		{
			return;
		}

		FVector Location;
		FRotator Rotation;
		ContextActor->GetActorEyesViewPoint(Location, Rotation);

		StateOut.Position = Location;
		StateOut.Orientation = Rotation.Quaternion();
		StateOut.HorizontalFOVDegrees = 90;
		StateOut.OrthoWorldCoordinateWidth = 1;
		StateOut.AspectRatio = 1.0;
		StateOut.bIsOrthographic = false;
		StateOut.bIsVR = true;
	}

	virtual EToolContextCoordinateSystem GetCurrentCoordinateSystem() const override
	{
		return ToolsComp->GetCurrentCoordinateSystem();
	}

	virtual EToolContextTransformGizmoMode GetCurrentTransformGizmoMode() const override
	{
		return ToolsComp->GetCurrentTransformGizmoMode();
	}

	virtual FToolContextSnappingConfiguration GetCurrentSnappingSettings() const override
	{
		return FToolContextSnappingConfiguration();
	}

	virtual UMaterialInterface* GetStandardMaterial(EStandardToolContextMaterials MaterialType) const override
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}

	virtual FViewport* GetHoveredViewport() const override
	{
		return ActiveViewport;
	}

	virtual FViewport* GetFocusedViewport() const override
	{
		return ActiveViewport;
	}

protected:
	UInteractiveToolsContext* ToolsContext;
	UXRCreativeITFComponent* ToolsComp;
	APawn* ContextActor = nullptr;
	FViewport* ActiveViewport = nullptr;
};


//////////////////////////////////////////////////////////////////////////


class FXRCreativeToolsFrameworkRenderImpl : public IToolsContextRenderAPI
{
public:
	UXRCreativeITFRenderComponent* RenderComponent;
	TSharedPtr<FPrimitiveDrawInterface> PDI;
	const FSceneView* SceneView;
	FViewCameraState ViewCameraState;

	FXRCreativeToolsFrameworkRenderImpl(UXRCreativeITFRenderComponent* RenderComponentIn, const FSceneView* ViewIn, FViewCameraState CameraState)
		: RenderComponent(RenderComponentIn), SceneView(ViewIn), ViewCameraState(CameraState)
	{
		PDI = RenderComponentIn->GetPDIForView(ViewIn);
	}

	virtual FPrimitiveDrawInterface* GetPrimitiveDrawInterface() override
	{
		return PDI.Get();
	}

	virtual const FSceneView* GetSceneView() override
	{
		return SceneView;
	}

	virtual FViewCameraState GetCameraState() override
	{
		return ViewCameraState;
	}

	virtual EViewInteractionState GetViewInteractionState() override
	{
		return EViewInteractionState::Focused;
	}
};


//////////////////////////////////////////////////////////////////////////


class FXRCreativeToolsContextTransactionImpl : public IToolsContextTransactionsAPI
{
public:
	FXRCreativeToolsContextTransactionImpl(UInteractiveToolsContext* InToolsContext, UTypedElementSelectionSet* InSelectionSet, bool bInEditor)
		: WeakToolsContext(InToolsContext)
		, WeakSelectionSet(InSelectionSet)
		, bEditor(bInEditor)
	{
		ensure(InToolsContext);
		ensure(InSelectionSet);

#if !WITH_EDITOR
		if (!ensure(bInEditor == false))
		{
			bEditor = false;
		}
#endif
	}

	virtual void DisplayMessage(const FText& Message, EToolMessageLevel Level) override
	{
		if (UInteractiveToolsContext* ToolsContext = WeakToolsContext.Get())
		{
			if (Level == EToolMessageLevel::UserNotification || Level == EToolMessageLevel::UserMessage)
			{
				ToolsContext->PostToolNotificationMessage(Message);
			}
			else if (Level == EToolMessageLevel::UserWarning || Level == EToolMessageLevel::UserError)
			{
				ToolsContext->PostToolWarningMessage(Message);
			}
		}

		ELogVerbosity::Type Verbosity;

		switch (Level)
		{
			case EToolMessageLevel::UserError:        Verbosity = ELogVerbosity::Error;   break;
			case EToolMessageLevel::UserWarning:      Verbosity = ELogVerbosity::Warning; break;
			case EToolMessageLevel::UserNotification: Verbosity = ELogVerbosity::Display; break;
			case EToolMessageLevel::UserMessage:      Verbosity = ELogVerbosity::Log;     break;
			case EToolMessageLevel::Internal:         Verbosity = ELogVerbosity::Verbose; break;
			default:
				ensureMsgf(false, TEXT("Unhandled EToolMessageLevel: %X"), EnumToUnderlyingType(Level));
				Verbosity = ELogVerbosity::Error;
				break;
		}

#if !NO_LOGGING
		FMsg::Logf(__FILE__, __LINE__, LogXRCreative.GetCategoryName(), Verbosity,
			TEXT("IToolsContextTransactionsAPI::DisplayMessage: %s"), *Message.ToString());
#endif
	}

	virtual void PostInvalidation() override
	{
		// TODO?
	}

	virtual void BeginUndoTransaction(const FText& Description) override
	{
		bInTransaction = true;

#if WITH_EDITOR
		if (bEditor)
		{
			GEditor->BeginTransaction(Description);
		}
		else
#endif
		{
			// TODO
		}
	}

	virtual void EndUndoTransaction() override
	{
#if WITH_EDITOR
		if (bEditor)
		{
			GEditor->EndTransaction();
		}
		else
#endif
		{
			// TODO
		}

		bInTransaction = false;
	}
	
	virtual void CancelUndoTransaction() override
	{
#if WITH_EDITOR
		if (bEditor)
		{
			GEditor->CancelTransaction(0);
		}
		else
#endif
		{
			// TODO
		}

		bInTransaction = false;
	}

	virtual void AppendChange(UObject* TargetObject, TUniquePtr<FToolCommandChange> Change, const FText& Description) override
	{
#if WITH_EDITOR
		if (bEditor)
		{
			FScopedTransaction Transaction(Description);
			if (ensure(GUndo))
			{
				GUndo->StoreUndo(TargetObject, MoveTemp(Change));
			}
		}
		else
#endif
		{
			bool bCloseTransaction = false;
			if (!bInTransaction)
			{
				BeginUndoTransaction(Description);
				bCloseTransaction = true;
			}

			// TODO

			if (bCloseTransaction)
			{
				EndUndoTransaction();
			}
		}
	}

	virtual bool RequestSelectionChange(const FSelectedObjectsChangeList& SelectionChange) override
	{
		if (UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get())
		{
			if (SelectionChange.ModificationType == ESelectedObjectsModificationType::Clear)
			{
				SelectionSet->ClearSelection(FTypedElementSelectionOptions());
				return true;
			}

			TArray<FTypedElementHandle> ChangeElements;
			ChangeElements.Reserve(SelectionChange.Actors.Num() + SelectionChange.Components.Num());

#if WITH_EDITOR
			if (bEditor)
			{
				for (AActor* Actor : SelectionChange.Actors)
				{
					ChangeElements.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
				}

				for (UActorComponent* Comp : SelectionChange.Components)
				{
					ChangeElements.Add(UEngineElementsLibrary::AcquireEditorComponentElementHandle(Comp));
				}
			}
			else
#endif
			{
				// TODO
			}

			switch (SelectionChange.ModificationType)
			{
			case ESelectedObjectsModificationType::Add:
				SelectionSet->SelectElements(ChangeElements, FTypedElementSelectionOptions());
				return true;
			case ESelectedObjectsModificationType::Replace:
				SelectionSet->SetSelection(ChangeElements, FTypedElementSelectionOptions());
				return true;
			case ESelectedObjectsModificationType::Remove:
				SelectionSet->DeselectElements(ChangeElements, FTypedElementSelectionOptions());
				return true;
			default:
				checkNoEntry();
			}
		}

		ensure(false);
		return false;
	}

protected:
	TWeakObjectPtr<UInteractiveToolsContext> WeakToolsContext;
	TWeakObjectPtr<UTypedElementSelectionSet> WeakSelectionSet;
	bool bEditor = false;
	bool bInTransaction = false;
};


//////////////////////////////////////////////////////////////////////////


class FXRCreativeTransformGizmoActorFactory : public FCombinedTransformGizmoActorFactory
{
public:
	FXRCreativeTransformGizmoActorFactory(
		UGizmoViewContext* InGizmoViewContext,
		UXRCreativeITFComponent* InITFComponent
	)
		: FCombinedTransformGizmoActorFactory(InGizmoViewContext)
		, ITFComponent(InITFComponent)
	{
		ensure(InGizmoViewContext);
		ensure(InITFComponent);
	}

	virtual ACombinedTransformGizmoActor* CreateNewGizmoActor(UWorld* World) const override
	{
		TSubclassOf<AXRCreativeCombinedTransformGizmoActor> GizmoClass =
			ITFComponent->GetCombinedGizmoActorClass();

		AXRCreativeCombinedTransformGizmoActor* Gizmo =
			World->SpawnActor<AXRCreativeCombinedTransformGizmoActor>(GizmoClass);

		if (!GizmoClass || !ensure(Gizmo))
		{
			// Fall back to base
			return FCombinedTransformGizmoActorFactory::CreateNewGizmoActor(World);
		}

		Gizmo->SetEnabledElements(EnableElements);

		TInlineComponentArray<UXRCreativeGizmoMeshComponent*> GizmoMeshes;
		Gizmo->GetComponents(GizmoMeshes);
		for (UXRCreativeGizmoMeshComponent* GizmoMesh : GizmoMeshes)
		{
			GizmoMesh->Initialize();
		}

		ITFComponent->GizmoActor = Gizmo;

		Gizmo->OwnerAvatar = Cast<AXRCreativeAvatar>(ITFComponent->GetOwner());
		
		return Gizmo;
	}

protected:
	UXRCreativeITFComponent* ITFComponent;
};


//////////////////////////////////////////////////////////////////////////


UXRCreativeITFComponent::UXRCreativeITFComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bTickEvenWhenPaused = true;

	bAutoActivate = true;
	bWantsInitializeComponent = true;

	UnselectableActorClasses.Add(AXRCreativeAvatar::StaticClass());
}


void UXRCreativeITFComponent::SetPointerComponent(UXRCreativePointerComponent* InPointer)
{
	PointerComponent = InPointer;
}


void UXRCreativeITFComponent::InitializeComponent()
{
	Super::InitializeComponent();

#if WITH_EDITOR
	if (GEditor)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransBuffer->OnUndo().AddUObject(this, &UXRCreativeITFComponent::HandleTransactorUndo);
			TransBuffer->OnRedo().AddUObject(this, &UXRCreativeITFComponent::HandleTransactorRedo);
		}
	}
#endif

	ensure(IsValid(PointerComponent));

	// Ensure our motion controller components tick before we do
	PrimaryComponentTick.AddPrerequisite(PointerComponent, PointerComponent->PrimaryComponentTick);

	if (!GetSelectionSet())
	{
		SelectionSet = NewObject<UTypedElementSelectionSet>(this, NAME_None, RF_Transactional);
	}

	//////////////////////////////////////////////////////////////////////////
	// Interactive Tools Framework init
	ToolsContext = NewObject<UInteractiveToolsContext>(this);
	
	ContextQueriesAPI = MakeShared<FXRCreativeToolsContextQueriesImpl>(ToolsContext, this);
	ContextTransactionsAPI = MakeShared<FXRCreativeToolsContextTransactionImpl>(ToolsContext, GetSelectionSet(), IsInEditor());

	ToolsContext->Initialize(ContextQueriesAPI.Get(), ContextTransactionsAPI.Get());

#if 0
	// create scene history
	SceneHistory = NewObject<USceneHistoryManager>(this);
	SceneHistory->OnHistoryStateChange.AddUObject(this, &UXRCreativeToolsSubsystem::OnSceneHistoryStateChange);
#endif

	// register selection interaction
	{
		auto InteractionActorCallback = [this](AActor* SelectionCandidate)
			{
				if (HaveActiveTool() || !PointerComponent->IsEnabled())
				{
					return false;
				}

				if (CanSelectPredicate.IsBound())
				{
					return CanSelectPredicate.Execute(SelectionCandidate);
				}

				if (SelectionCandidate)
				{
					for (const TSubclassOf<AActor>& DisallowedClass : UnselectableActorClasses)
					{
						if (DisallowedClass.Get() && SelectionCandidate->IsA(DisallowedClass))
						{
							return false;
						}
					}
				}

				return true;
			};

		auto InteractionTraceCallback = [this](const FInputDeviceRay& InRay) -> FHitResult
			{
				FHitResult Result;
				if (ensure(PointerComponent))
				{
					GetWorld()->LineTraceSingleByChannel(Result, InRay.WorldRay.Origin,
						InRay.WorldRay.PointAt(UXRCreativeSelectionInteraction::RayLength),
						ECC_Visibility, PointerComponent->GetQueryParams());
				}
				return Result;
			};

		SelectionInteraction = NewObject<UXRCreativeSelectionInteraction>(this);
		SelectionInteraction->Initialize(GetSelectionSet(),
			MoveTemp(InteractionActorCallback),
			MoveTemp(InteractionTraceCallback));
		ToolsContext->InputRouter->RegisterSource(SelectionInteraction);
	}

	// create transform interaction
	UGizmoViewContext* GizmoViewContext =
		ToolsContext->GizmoManager->GetContextObjectStore()->FindContext<UGizmoViewContext>();

	TSharedRef<FXRCreativeTransformGizmoActorFactory> GizmoActorFactory =
		MakeShared<FXRCreativeTransformGizmoActorFactory>(GizmoViewContext, this);

	TransformInteraction = NewObject<UXRCreativeTransformInteraction>(this);
	TransformInteraction->Initialize(
		GizmoActorFactory,
		GetSelectionSet(),
		ToolsContext->GizmoManager,
		[this]() { return HaveActiveTool() == false; }
	);
	TransformInteraction->ForceUpdateGizmoState();

	// create PDI rendering bridge Component
	PDIRenderComponent = NewObject<UXRCreativeITFRenderComponent>(GetOwner());
	PDIRenderComponent->SetupAttachment(GetOwner()->GetRootComponent());
	PDIRenderComponent->RegisterComponent();

	// register transform gizmo util helper
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(ToolsContext);
}


void UXRCreativeITFComponent::UninitializeComponent()
{
	bIsShuttingDown = true;

#if WITH_EDITOR
	if (GEditor)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransBuffer->OnUndo().RemoveAll(this);
			TransBuffer->OnRedo().RemoveAll(this);
		}
	}
#endif

	if (ToolsContext)
	{
		//CancelOrCompleteActiveTool();

		TransformInteraction->Shutdown();
		SelectionInteraction->Shutdown();

		// unregister transform gizmo helper
		UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(ToolsContext);

		ToolsContext->Shutdown();
	}

	ToolsContext = nullptr;

	ContextQueriesAPI = nullptr;
	ContextTransactionsAPI = nullptr;

	SelectionInteraction = nullptr;
	TransformInteraction = nullptr;

	bIsShuttingDown = false;

	Super::UninitializeComponent();
}


bool UXRCreativeITFComponent::IsInEditor() const
{
#if WITH_EDITOR
	if (IVREditorModule::IsAvailable())
	{
		UVREditorModeBase* VrMode = IVREditorModule::Get().GetVRModeBase();
		if (VrMode && (VrMode->GetWorld() == this->GetWorld()))
		{
			return true;
		}
	}
#endif

	return false;
}


bool UXRCreativeITFComponent::CanUndo() const
{
#if WITH_EDITOR
	if (IsInEditor() && GUnrealEd && GUnrealEd->Trans)
	{
		return GUnrealEd->Trans->CanUndo() && FSlateApplication::Get().IsNormalExecution();
	}
#endif

	return false;
}


bool UXRCreativeITFComponent::CanRedo() const
{
#if WITH_EDITOR
	if (IsInEditor() && GUnrealEd && GUnrealEd->Trans)
	{
		return GUnrealEd->Trans->CanRedo() && FSlateApplication::Get().IsNormalExecution();
	}
#endif

	return false;
}


void UXRCreativeITFComponent::Undo()
{
#if WITH_EDITOR
	if (IsInEditor() && GUnrealEd)
	{
		GUnrealEd->Exec(GetWorld(), TEXT("TRANSACTION UNDO"));
	}
#endif
}


void UXRCreativeITFComponent::Redo()
{
#if WITH_EDITOR
	if (IsInEditor() && GUnrealEd)
	{
		GUnrealEd->Exec(GetWorld(), TEXT("TRANSACTION REDO"));
	}
#endif
}


void UXRCreativeITFComponent::TickComponent(float InDeltaTime, enum ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction)
{
	Super::TickComponent(InDeltaTime, InTickType, InThisTickFunction);

	if (!IsActive())
	{
		return;
	}

	if (InTickType == LEVELTICK_ViewportsOnly)
	{
#if WITH_EDITOR
		if (IsInEditor())
		{
			FEditorScriptExecutionGuard ScriptGuard;
			EditorToolsTick(InDeltaTime);
		}
#endif
		return;
	}

	ToolsTick(InDeltaTime);
}

void UXRCreativeITFComponent::ToolsTick(float InDeltaTime)
{
	UGameViewportClient* ViewportClient = GetWorld()->GetGameViewport();
	if (!ensure(ViewportClient))
	{
		return;
	}

	FInputDeviceState InputState = CurrentMouseState;
	InputState.InputDevice = EInputDevices::Mouse;

	FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();
	FModifierKeysState ModifierState = FSlateApplication::Get().GetModifierKeys();

	// update modifier keys
	InputState.SetModifierKeyStates(
		ModifierState.IsLeftShiftDown(),
		ModifierState.IsAltDown(),
		ModifierState.IsControlDown(),
		ModifierState.IsCommandDown());

#ifdef XRCREATIVE_CALC_ITF_MOUSE_2D
	TSharedPtr<IGameLayerManager> LayerManager = ViewportClient->GetGameLayerManager();
	FGeometry ViewportGeometry;
	if (ensure(LayerManager.IsValid()))
	{
		ViewportGeometry = LayerManager->GetViewportWidgetHostGeometry();
	}

	// why do we need this scale here? what is it for?
	FVector2D ViewportMousePos = ViewportGeometry.Scale * ViewportGeometry.AbsoluteToLocal(MousePosition);
#endif // #ifdef XRCREATIVE_CALC_ITF_MOUSE_2D

	FSceneViewport* Viewport = ViewportClient->GetGameViewport();

	FEngineShowFlags* ShowFlags = ViewportClient->GetEngineShowFlags();
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport,
		GetWorld()->Scene,
		*ShowFlags)
		.SetRealtimeUpdate(true));

	APawn* OwnerPawn = CastChecked<APawn>(GetOwner());
	APlayerController* PC = CastChecked<APlayerController>(OwnerPawn->GetController());
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PC->Player);
	FVector ViewLocation;
	FRotator ViewRotation;
	const int32 StereoViewIndex = GEngine->IsStereoscopic3D(Viewport) ? eSSE_LEFT_EYE : eSSE_MONOSCOPIC;
	FSceneView* SceneView = LocalPlayer->CalcSceneView(&ViewFamily, ViewLocation, ViewRotation,
		LocalPlayer->ViewportClient->Viewport, nullptr, StereoViewIndex);
	if (!SceneView)
	{
		return;
	}

	UGizmoViewContext* GizmoViewContext = ToolsContext->ContextObjectStore->FindContext<UGizmoViewContext>();
	if (GizmoViewContext)
	{
		GizmoViewContext->ResetFromSceneView(*SceneView);
	}

	ContextQueriesAPI->UpdateActiveViewport(Viewport);

	CurrentViewCameraState.Position = ViewLocation;
	CurrentViewCameraState.Orientation = ViewRotation.Quaternion();
	CurrentViewCameraState.HorizontalFOVDegrees = SceneView->FOV;
	CurrentViewCameraState.AspectRatio = Viewport->GetDesiredAspectRatio(); //ViewportClient->AspectRatio;
	CurrentViewCameraState.bIsOrthographic = false;
	CurrentViewCameraState.bIsVR = true;
	CurrentViewCameraState.OrthoWorldCoordinateWidth = 1;

#ifdef XRCREATIVE_CALC_ITF_MOUSE_2D
	FVector4 ScreenPos = SceneView->PixelToScreen(ViewportMousePos.X, ViewportMousePos.Y, 0);

	const FMatrix InvViewMatrix = SceneView->ViewMatrices.GetViewToWorld();
	const FMatrix InvProjMatrix = SceneView->ViewMatrices.GetClipToView();

	const float ScreenX = ScreenPos.X;
	const float ScreenY = ScreenPos.Y;

	FVector Origin;
	FVector Direction;
	if (!ViewportClient->IsOrtho())
	{
		Origin = SceneView->ViewMatrices.GetViewOrigin();
		Direction = InvViewMatrix.TransformVector(FVector(InvProjMatrix.TransformFVector4(FVector4(ScreenX * GNearClippingPlane, ScreenY * GNearClippingPlane, 0.0f, GNearClippingPlane)))).GetSafeNormal();
	}
	else
	{
		Origin = InvViewMatrix.TransformFVector4(InvProjMatrix.TransformFVector4(FVector4(ScreenX, ScreenY, 0.5f, 1.0f)));
		Direction = InvViewMatrix.TransformVector(FVector(0, 0, 1)).GetSafeNormal();
	}

	// fudge factor so we don't hit actor...
	Origin += 1.0 * Direction;

	InputState.Mouse.Position2D = ViewportMousePos;
	InputState.Mouse.Delta2D = CurrentMouseState.Mouse.Position2D - PrevMousePosition;
	PrevMousePosition = InputState.Mouse.Position2D;
	//InputState.Mouse.WorldRay = FRay(Origin, Direction);
#endif // #ifdef XRCREATIVE_CALC_ITF_MOUSE_2D

	const FVector PointerDirection = (PointerComponent->GetFilteredTraceEnd(false) - PointerComponent->GetComponentLocation()).GetSafeNormal();
	InputState.Mouse.WorldRay = FRay(PointerComponent->GetComponentLocation(), PointerDirection, true);

	if (bPendingMouseStateChange || ToolsContext->InputRouter->HasActiveMouseCapture())
	{
		ToolsContext->InputRouter->PostInputEvent(InputState);
	}
	else
	{
		ToolsContext->InputRouter->PostHoverInputEvent(InputState);
	}

	// clear down or up flags now that we have sent event
	if (bPendingMouseStateChange)
	{
		if (CurrentMouseState.Mouse.Left.bDown)
		{
			CurrentMouseState.Mouse.Left.SetStates(false, true, false);
		}
		else
		{
			CurrentMouseState.Mouse.Left.SetStates(false, false, false);
		}
		bPendingMouseStateChange = false;
	}

	// tick things
	ToolsContext->ToolManager->Tick(InDeltaTime);
	ToolsContext->GizmoManager->Tick(InDeltaTime);

	// render things
	FXRCreativeToolsFrameworkRenderImpl RenderAPI(PDIRenderComponent, SceneView, CurrentViewCameraState);
	ToolsContext->ToolManager->Render(&RenderAPI);
	ToolsContext->GizmoManager->Render(&RenderAPI);

	// force rendering flush so that PDI lines get drawn
	FlushRenderingCommands();
}


#if WITH_EDITOR
void UXRCreativeITFComponent::EditorToolsTick(float InDeltaTime)
{
	UVREditorModeBase* Mode = IVREditorModule::Get().GetVRModeBase();
	if (!ensure(Mode))
	{
		return;
	}

	TSharedPtr<SLevelViewport> LevelViewport = Mode->GetVrLevelViewport();
	if (!ensure(LevelViewport))
	{
		return;
	}

	FInputDeviceState InputState = CurrentMouseState;
	InputState.InputDevice = EInputDevices::Mouse;

	FModifierKeysState ModifierState = FSlateApplication::Get().GetModifierKeys();
	InputState.SetModifierKeyStates(
		ModifierState.IsLeftShiftDown(),
		ModifierState.IsAltDown(),
		ModifierState.IsControlDown(),
		ModifierState.IsCommandDown());


	TSharedPtr<FEditorViewportClient> ViewportClient = LevelViewport->GetViewportClient();
	FViewport* Viewport = ViewportClient->Viewport;

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport,
		GetWorld()->Scene,
		ViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(true));

	FSceneView* SceneView = ViewportClient->CalcSceneView(&ViewFamily, EStereoscopicEye::eSSE_LEFT_EYE);
	if (!ensure(SceneView))
	{
		return;
	}

	UGizmoViewContext* GizmoViewContext = ToolsContext->ContextObjectStore->FindContext<UGizmoViewContext>();
	if (GizmoViewContext)
	{
		GizmoViewContext->ResetFromSceneView(*SceneView);
	}

	ContextQueriesAPI->UpdateActiveViewport(Viewport);

	CurrentViewCameraState.Position = SceneView->ViewLocation;
	CurrentViewCameraState.Orientation = SceneView->ViewRotation.Quaternion();
	CurrentViewCameraState.HorizontalFOVDegrees = SceneView->FOV;
	CurrentViewCameraState.AspectRatio = Viewport->GetDesiredAspectRatio();
	CurrentViewCameraState.bIsOrthographic = false;
	CurrentViewCameraState.bIsVR = true;
	CurrentViewCameraState.OrthoWorldCoordinateWidth = 1;

#ifdef XRCREATIVE_CALC_ITF_MOUSE_2D
	const int32 MouseX = ViewportClient->GetCachedMouseX();
	const int32 MouseY = ViewportClient->GetCachedMouseY();

	InputState.Mouse.Position2D = FVector2D(MouseX, MouseY);
	InputState.Mouse.Delta2D = CurrentMouseState.Mouse.Position2D - PrevMousePosition;
	PrevMousePosition = InputState.Mouse.Position2D;

	FViewportCursorLocation MouseViewportRay(SceneView, ViewportClient.Get(), MouseX, MouseY);
	FVector RayOrigin = MouseViewportRay.GetOrigin();
	FVector RayDirection = MouseViewportRay.GetDirection();
	//InputState.Mouse.WorldRay = FRay(RayOrigin, RayDirection);
#endif // #ifdef XRCREATIVE_CALC_ITF_MOUSE_2D

	const FVector PointerDirection = (PointerComponent->GetFilteredTraceEnd(false) - PointerComponent->GetComponentLocation()).GetSafeNormal();
	InputState.Mouse.WorldRay = FRay(PointerComponent->GetComponentLocation(), PointerDirection, true);

	if (bPendingMouseStateChange || ToolsContext->InputRouter->HasActiveMouseCapture())
	{
		ToolsContext->InputRouter->PostInputEvent(InputState);
	}
	else
	{
		ToolsContext->InputRouter->PostHoverInputEvent(InputState);
	}

	// clear down or up flags now that we have sent event
	if (bPendingMouseStateChange)
	{
		if (CurrentMouseState.Mouse.Left.bDown)
		{
			CurrentMouseState.Mouse.Left.SetStates(false, true, false);
		}
		else
		{
			CurrentMouseState.Mouse.Left.SetStates(false, false, false);
		}
		bPendingMouseStateChange = false;
	}

	// tick things
	ToolsContext->ToolManager->Tick(InDeltaTime);
	ToolsContext->GizmoManager->Tick(InDeltaTime);

	// render things
	FXRCreativeToolsFrameworkRenderImpl RenderAPI(PDIRenderComponent, SceneView, CurrentViewCameraState);
	ToolsContext->ToolManager->Render(&RenderAPI);
	ToolsContext->GizmoManager->Render(&RenderAPI);

	// force rendering flush so that PDI lines get drawn
	FlushRenderingCommands();
}


void UXRCreativeITFComponent::HandleTransactorUndo(const FTransactionContext& TransactionContext, bool bSucceeded)
{
	if (bSucceeded)
	{
		OnUndo.Broadcast();
	}
}


void UXRCreativeITFComponent::HandleTransactorRedo(const FTransactionContext& TransactionContext, bool bSucceeded)
{
	if (bSucceeded)
	{
		OnRedo.Broadcast();
	}
}
#endif // #if WITH_EDITOR


void UXRCreativeITFComponent::LeftMousePressed()
{
	if (!IsActive())
	{
		return;
	}

	CurrentMouseState.Mouse.Left.SetStates(true, false, false);
	bPendingMouseStateChange = true;
}


void UXRCreativeITFComponent::LeftMouseReleased()
{
	if (!IsActive())
	{
		return;
	}

	CurrentMouseState.Mouse.Left.SetStates(false, false, true);
	bPendingMouseStateChange = true;
}


UTypedElementSelectionSet* UXRCreativeITFComponent::GetSelectionSet() const
{
	if (SelectionSet)
	{
		return SelectionSet;
	}

#if WITH_EDITOR
	if (IsInEditor())
	{
		ULevelEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();
		return EditorSubsystem->GetSelectionSet();
	}
#endif

	return nullptr;
}


bool UXRCreativeITFComponent::HaveActiveTool()
{
	if (!ToolsContext || !ToolsContext->ToolManager)
	{
		return false;
	}

	return ToolsContext->ToolManager->HasActiveTool(EToolSide::Left)
		|| ToolsContext->ToolManager->HasActiveTool(EToolSide::Right);
}


void UXRCreativeITFComponent::SetCurrentCoordinateSystem(EToolContextCoordinateSystem CoordSystem)
{
	if(IsValid(GetSelectionSet()))
	{
		CurrentCoordinateSystem = CoordSystem; 
		TransformInteraction->ForceUpdateGizmoState();	
	}

}


void UXRCreativeITFComponent::SetCurrentTransformGizmoMode(EToolContextTransformGizmoMode GizmoMode)
{
	CurrentTransformGizmoMode = GizmoMode;
	TransformInteraction->ForceUpdateGizmoState();
}


void UXRCreativeITFComponent::RecreateActorPhysicsState(AActor* Actor)
{
#if WITH_EDITOR
	if (!Actor)
	{
		return;
	}

	for (UActorComponent* Comp : Actor->GetComponents())
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Comp))
		{
			if (PrimComp->IsRegistered() && PrimComp->IsPhysicsStateCreated())
			{
				PrimComp->RecreatePhysicsState();
			}
		}
	}
#endif
}
