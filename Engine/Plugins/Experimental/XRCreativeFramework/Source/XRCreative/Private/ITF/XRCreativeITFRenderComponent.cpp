// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeITFRenderComponent.h"
#include "MeshElementCollector.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"


namespace UE::XRCreative::Private
{
	// SceneProxy for UXRCreativeITFRenderComponent. Just uses the PDIs available in GetDynamicMeshElements
	// to draw the lines/points accumulated by the Component.
	class FRenderComponentSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FRenderComponentSceneProxy(
			const UXRCreativeITFRenderComponent* InComponent,
			TUniqueFunction<void(TArray<UXRCreativeITFRenderComponent::FPDILine>&, TArray<UXRCreativeITFRenderComponent::FPDIPoint>&)>&& GeometryQueryFunc)
			: FPrimitiveSceneProxy(InComponent)
		{
			GetGeometryQueryFunc = MoveTemp(GeometryQueryFunc);

			// GetDynamicMeshElements needs to run single pass, because the function GetGeometryQueryFunc consumes cached geometry
			// generated over the previous frame, and the geometry won't exist if called a second time.
			bSinglePassGDME = true;
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			TArray<UXRCreativeITFRenderComponent::FPDILine> Lines;
			TArray<UXRCreativeITFRenderComponent::FPDIPoint> Points;
			GetGeometryQueryFunc(Lines, Points);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					int32 NumLines = Lines.Num(), NumPoints = Points.Num();
					for (int32 k = 0; k < NumLines; ++k)
					{
						PDI->DrawLine(Lines[k].Start, Lines[k].End, Lines[k].Color, Lines[k].DepthPriorityGroup, Lines[k].Thickness, Lines[k].DepthBias, Lines[k].bScreenSpace);
					}
					for (int32 k = 0; k < NumPoints; ++k)
					{
						PDI->DrawPoint(Points[k].Position, Points[k].Color, Points[k].PointSize, Points[k].DepthPriorityGroup);
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View);
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = false;
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			Result.bRenderCustomDepth = ShouldRenderCustomDepth();
			return Result;
		}

		//virtual bool CanBeOccluded() const override
		//{
		//	return false;
		//}

		virtual uint32 GetMemoryFootprint(void) const override { return sizeof * this + GetAllocatedSize(); }
		uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }


		// set to lambda that steals current lines/points from the Component
		TUniqueFunction<void(TArray<UXRCreativeITFRenderComponent::FPDILine>&, TArray<UXRCreativeITFRenderComponent::FPDIPoint>&)> GetGeometryQueryFunc;
	};


	// implementation of FPrimitiveDrawInterface that forwards DrawLine/DrawPoint calls to
	// a UXRCreativeITFRenderComponent instance. No other PDI functionality is implemented.
	// Instances of this class are created by GetPDIForView() below.
	class FRenderComponentPDI : public FPrimitiveDrawInterface
	{
	public:
		UXRCreativeITFRenderComponent* RenderComponent;

		FRenderComponentPDI(const FSceneView* InView, UXRCreativeITFRenderComponent* RenderComponentIn) : FPrimitiveDrawInterface(InView)
		{
			RenderComponent = RenderComponentIn;
		}

		virtual bool IsHitTesting() { return false; }
		virtual void SetHitProxy(HHitProxy* HitProxy) { };
		virtual void RegisterDynamicResource(FDynamicPrimitiveResource* DynamicResource) { ensure(false); }
		virtual void AddReserveLines(uint8 DepthPriorityGroup, int32 NumLines, bool bDepthBiased = false, bool bThickLines = false) { ensure(false); }
		virtual int32 DrawMesh(const FMeshBatch& Mesh) { ensure(false); return 0; }
		virtual void DrawSprite(
			const FVector& Position, float SizeX, float SizeY,
			const FTexture* Sprite, const FLinearColor& Color, uint8 DepthPriorityGroup,
			float U, float UL, float V, float VL, uint8 BlendMode = 1 /*SE_BLEND_Masked*/, float OpacityMaskRefVal = 0.5f) {
			ensure(false);
		}

		virtual void DrawLine(const FVector& Start, const FVector& End, const FLinearColor& Color,
			uint8 DepthPriorityGroup, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false)
		{
			RenderComponent->DrawLine(Start, End, Color, DepthPriorityGroup, Thickness, DepthBias, bScreenSpace);
		}

		virtual void DrawTranslucentLine(const FVector& Start, const FVector& End, const FLinearColor& Color,
			uint8 DepthPriorityGroup, float Thickness = 0.0f, float DepthBias = 0.0f, bool bScreenSpace = false) override
		{
			RenderComponent->DrawLine(Start, End, Color, DepthPriorityGroup, Thickness, DepthBias, bScreenSpace);
		}

		virtual void DrawPoint(const FVector& Position, const FLinearColor& Color, float PointSize, uint8 DepthPriorityGroup)
		{
			RenderComponent->DrawPoint(Position, Color, PointSize, DepthPriorityGroup);
		}
	};

} // namespace UE::XRCreative::Private


TSharedPtr<FPrimitiveDrawInterface> UXRCreativeITFRenderComponent::GetPDIForView(const FSceneView* InView)
{
	return MakeShared<UE::XRCreative::Private::FRenderComponentPDI>(InView, this);
}


void UXRCreativeITFRenderComponent::DrawLine(
	const FVector& Start,
	const FVector& End,
	const FLinearColor& Color,
	uint8 DepthPriorityGroupIn,
	float Thickness,
	float DepthBias,
	bool bScreenSpace
)
{
	FScopeLock Lock(&GeometryLock);
	CurrentLines.Add(
		FPDILine{ Start, End, Color, DepthPriorityGroupIn, Thickness, DepthBias, bScreenSpace });
}


void UXRCreativeITFRenderComponent::DrawPoint(
	const FVector& Position,
	const FLinearColor& Color,
	float PointSize,
	uint8 DepthPriorityGroupIn
)
{
	FScopeLock Lock(&GeometryLock);
	CurrentPoints.Add(
		FPDIPoint{ Position, Color, PointSize, DepthPriorityGroupIn });
}


TUniqueFunction<void(TArray<UXRCreativeITFRenderComponent::FPDILine>&, TArray<UXRCreativeITFRenderComponent::FPDIPoint>&)> UXRCreativeITFRenderComponent::MakeGetCurrentGeometryQueryFunc()
{
	return [this](TArray<FPDILine>& LineStorage, TArray<FPDIPoint>& PointStorage)
	{
		FScopeLock Lock(&GeometryLock);
		LineStorage = MoveTemp(CurrentLines);
		PointStorage = MoveTemp(CurrentPoints);
	};
}


FPrimitiveSceneProxy* UXRCreativeITFRenderComponent::CreateSceneProxy()
{
	return new UE::XRCreative::Private::FRenderComponentSceneProxy(this, MakeGetCurrentGeometryQueryFunc());
}


bool UXRCreativeITFRenderComponent::LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params)
{
	// hit testing not supported
	return false;
}


FBoxSphereBounds UXRCreativeITFRenderComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// infinite bounds. TODO: accumulate actual bounds? will result in unstable depth sorting...
	float f = 9999999.0f;
	return FBoxSphereBounds(FBox(-FVector(f, f, f), FVector(f, f, f)));
}
