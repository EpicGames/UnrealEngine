// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "XRCreativeITFRenderComponent.generated.h"


class FPrimitiveDrawInterface;
class FSceneView;


/**
 * UXRCreativeITFRenderComponent is a helper component that can provide a
 * FPrimitiveDrawInterface API implementation. This PDI can be passed to UInterativeTool::Render()
 * and UInteractiveGizmo::Render() (via a IToolsContextRenderAPI implementation).
 * The UToolsContextRenderComponent will accumulate any DrawLine() and DrawPoint() requests
 * and then pass them to it's SceneProxy for rendering in the next frame.
 *
 * (in the UE Editor, those functions can be passed an Editor PDI that can draw immediately,
 *  but this is not possible at Runtime, so we use this accumulate-and-draw workaround)
 *
 * TODO: lines and points could be drawn via Line/Point Components, which would allow them
 * to be batched, providing much better performance.
 */
UCLASS()
class XRCREATIVE_API UXRCreativeITFRenderComponent : public UPrimitiveComponent
{
	GENERATED_BODY()
public:

	// parameters passed to a PDI DrawLine() call
	struct FPDILine
	{
		const FVector Start;
		const FVector End;
		const FLinearColor Color;
		uint8 DepthPriorityGroup;
		float Thickness;
		float DepthBias;
		bool bScreenSpace;
	};

	// parameters passed to a PDI DrawPoint() call
	struct FPDIPoint
	{
		FVector Position;
		FLinearColor Color;
		float PointSize;
		uint8 DepthPriorityGroup;
	};

public:

	/** @return a new FPrimitiveDrawInterface implementation allocated for the given FSceneView. See .cpp for details. */
	TSharedPtr<FPrimitiveDrawInterface> GetPDIForView(const FSceneView* InView);

	// mirrors FPrimitiveDrawInterface::DrawLine()
	virtual void DrawLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
	);

	// mirrors FPrimitiveDrawInterface::DrawPoint()
	virtual void DrawPoint(
		const FVector& Position,
		const FLinearColor& Color,
		float PointSize,
		uint8 DepthPriorityGroup
	);


protected:

	// set of lines populated by DrawLine calls
	TArray<FPDILine> CurrentLines;

	// set of points created by DrawPoint calls
	TArray<FPDIPoint> CurrentPoints;

	// protects CurrentLines and CurrentPoints
	FCriticalSection GeometryLock;

	// returns a lambda that will be passed to SceneProxy, which will then allow it to
	// steal values of CurrentLines and CurrentPoints
	TUniqueFunction<void(TArray<FPDILine>&, TArray<FPDIPoint>&)> MakeGetCurrentGeometryQueryFunc();

	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual bool LineTraceComponent(FHitResult& OutHit, const FVector Start, const FVector End, const FCollisionQueryParams& Params) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

	// proxy is defined in cpp
	friend class FToolsContextRenderComponentSceneProxy;
};
