// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module.h"
#include "XRCreativeLog.h"


#define LOCTEXT_NAMESPACE "FXRCreativeModule"


DEFINE_LOG_CATEGORY(LogXRCreative);


void FXRCreativeModule::StartupModule()
{
}


void FXRCreativeModule::ShutdownModule()
{
}


IMPLEMENT_MODULE(FXRCreativeModule, XRCreative)


#undef LOCTEXT_NAMESPACE
