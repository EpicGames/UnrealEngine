// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeAvatar.h"
#include "XRCreativeGameMode.h"
#include "XRCreativeITFComponent.h"
#include "XRCreativeLog.h"
#include "XRCreativePointerComponent.h"
#include "XRCreativeToolset.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/WidgetComponent.h"
#include "Components/WidgetInteractionComponent.h"
#include "Engine/Level.h"
#include "Engine/LocalPlayer.h"
#include "Engine/InputDelegateBinding.h"
#include "EnhancedInputComponent.h"
#include "EnhancedInputSubsystems.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerController.h"
#include "Haptics/HapticFeedbackEffect_Base.h"
#include "InputMappingContext.h"
#include "IXRTrackingSystem.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "MotionControllerComponent.h"
#include "XRMotionControllerBase.h"

#if WITH_EDITOR
#	include "Editor.h"
#	include "EnhancedInputEditorSubsystem.h"
#	include "IConcertClient.h"
#	include "IConcertClientSequencerManager.h"
#	include "IConcertSyncClient.h"
#	include "IMultiUserClientModule.h"
#	include "IVREditorModule.h"
#	include "VREditorModeBase.h"
#	include "XRCreativeSettings.h"
#endif


AXRCreativeAvatar::AXRCreativeAvatar(const FObjectInitializer& ObjectInitializer)
{
	USceneComponent* SceneRoot = CreateDefaultSubobject<USceneComponent>("Root");
	SetRootComponent(SceneRoot);

	LeftController = CreateDefaultSubobject<UMotionControllerComponent>("LeftController");
	LeftController->SetupAttachment(GetRootComponent());
	LeftController->bTickInEditor = true;
	LeftController->MotionSource = "Left";

	LeftControllerAim = CreateDefaultSubobject<UMotionControllerComponent>("LeftControllerAim");
	LeftControllerAim->SetupAttachment(GetRootComponent());
	LeftControllerAim->bTickInEditor = true;
	LeftControllerAim->MotionSource = "LeftAim";

	LeftControllerPointer = CreateDefaultSubobject<UXRCreativePointerComponent>("LeftControllerPointer");
	LeftControllerPointer->SetupAttachment(LeftControllerAim);
	LeftControllerPointer->bTickInEditor = true;

	LeftControllerModel = CreateDefaultSubobject<USkeletalMeshComponent>("LeftControllerSkeletalMeshComponent");
	LeftControllerModel->SetupAttachment(LeftController);
	LeftControllerModel->bTickInEditor = true;
	LeftControllerModel->SetCastShadow(false);

	RightController = CreateDefaultSubobject<UMotionControllerComponent>("RightController");
	RightController->SetupAttachment(GetRootComponent());
	RightController->bTickInEditor = true;
	RightController->MotionSource = "Right";

	RightControllerAim = CreateDefaultSubobject<UMotionControllerComponent>("RightControllerAim");
	RightControllerAim->SetupAttachment(GetRootComponent());
	RightControllerAim->bTickInEditor = true;
	RightControllerAim->MotionSource = "RightAim";

	RightControllerPointer = CreateDefaultSubobject<UXRCreativePointerComponent>("RightControllerPointer");
	RightControllerPointer->SetupAttachment(RightControllerAim);
	RightControllerPointer->bTickInEditor = true;

	RightControllerModel = CreateDefaultSubobject<USkeletalMeshComponent>("RightControllerSkeletalMeshComponent");
	RightControllerModel->SetupAttachment(RightController);
	RightControllerModel->bTickInEditor = true;
	RightControllerModel->SetCastShadow(false);

	MenuWidget = CreateDefaultSubobject<UWidgetComponent>("Widget");
	MenuWidget->SetupAttachment(LeftControllerAim);
	MenuWidget->bTickInEditor = true;
	MenuWidget->SetEditTimeUsable(true);
	MenuWidget->SetCastShadow(false);

	WidgetInteraction = CreateDefaultSubobject<UWidgetInteractionComponent>("WidgetInteraction");
	WidgetInteraction->SetupAttachment(RightControllerAim);
	WidgetInteraction->bTickInEditor = true;
	WidgetInteraction->bShowDebug = false;
	WidgetInteraction->InteractionDistance = 50.0;
	
	ToolsComponent = CreateDefaultSubobject<UXRCreativeITFComponent>("ToolsComponent");
	ToolsComponent->bTickInEditor = true;
	ToolsComponent->SetPointerComponent(RightControllerPointer);

	BaseEyeHeight = 0.0f;
}


void AXRCreativeAvatar::OnConstruction(const FTransform& InTransform)
{
	Super::OnConstruction(InTransform);

	if (!HasAnyFlags(RF_ClassDefaultObject) && !InputComponent)
	{
		InputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), TEXT("XRCreativeInput0"), RF_Transient);
		RegisterInputComponent();
		UInputDelegateBinding::BindInputDelegatesWithSubojects(this, InputComponent);
	}

	// Force any skeletal mesh components to update their animation if we are running in-editor
	TInlineComponentArray<USkeletalMeshComponent*> SkeletalMeshComponents;
	GetComponents(SkeletalMeshComponents);
	for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
	{
		SkeletalMeshComponent->SetUpdateAnimationInEditor(true);
	}

#if WITH_EDITOR
	UWorld* World = GetWorld();
	const bool bActorsInitialized = World && World->AreActorsInitialized();
	if (!bActorsInitialized && !ToolsComponent->HasBeenInitialized())
	{
		// We're in editor, and the base class won't initialize our components, so do it manually.
		ToolsComponent->InitializeComponent();
	}
	UXRCreativeEditorSettings* Settings = UXRCreativeEditorSettings::GetXRCreativeEditorSettings();
	UE_LOGF(LogXRCreative, Log, "Handedness: %ls", *UEnum::GetValueAsString(Settings->Handedness) );
	
	if (Settings->Handedness == EXRCreativeHandedness::Left)
	{
		ToolsComponent->SetPointerComponent(LeftControllerPointer);
	}
#endif

	if (!IsTemplate())
	{
#if WITH_EDITOR
		MultiUserStartup();
#endif
	}
}

void AXRCreativeAvatar::BeginDestroy()
{
	UnregisterInputComponent();

	if (!IsTemplate())
	{
#if WITH_EDITOR
		MultiUserShutdown();
#endif
	}

	Super::BeginDestroy();
}


void AXRCreativeAvatar::Tick(float InDeltaSeconds)
{
	Super::Tick(InDeltaSeconds);

	ProcessHaptics(InDeltaSeconds);
}


void AXRCreativeAvatar::BeginPlay()
{
	if (AXRCreativeGameMode* GameMode = Cast<AXRCreativeGameMode>(GetWorld()->GetAuthGameMode()))
	{
		if (UXRCreativeToolset* ModeToolset = GameMode->GetToolset())
		{
			ConfigureToolset(ModeToolset);
		}
	}

	Super::BeginPlay();
}


void AXRCreativeAvatar::GetActorEyesViewPoint(FVector& Location, FRotator& Rotation) const
{
	FTransform HeadTransform = GetHeadTransform();
	Location = HeadTransform.GetLocation();
	Rotation = HeadTransform.GetRotation().Rotator();
}


void AXRCreativeAvatar::ConfigureToolset(UXRCreativeToolset* InToolset)
{
	check(InToolset);
	ensure(Toolset == nullptr);

	Toolset = InToolset;

	for (const FXRCreativeToolEntry& ToolEntry : Toolset->Tools)
	{
		if (ensure(ToolEntry.ToolClass))
		{
			UXRCreativeTool* NewTool = NewObject<UXRCreativeTool>(this, ToolEntry.ToolClass);
			Tools.Add(NewTool);
		}
	}
}


FTransform AXRCreativeAvatar::GetHeadTransform() const
{
	return GetHeadTransformRoomSpace() * GetActorTransform();
}


FTransform AXRCreativeAvatar::GetHeadTransformRoomSpace() const
{
	FQuat RoomSpaceHeadOrientation;
	FVector RoomSpaceHeadLocation;
	if (GEngine && GEngine->XRSystem &&
		GEngine->XRSystem->GetCurrentPose(IXRTrackingSystem::HMDDeviceId, RoomSpaceHeadOrientation, RoomSpaceHeadLocation))
	{
		return FTransform(
			RoomSpaceHeadOrientation,
			RoomSpaceHeadLocation,
			FVector(1.0f));
	}

	return FTransform();
}


bool AXRCreativeAvatar::GetLaserForHand(EControllerHand InHand, FVector& OutLaserStart, FVector& OutLaserEnd) const
{
	UXRCreativePointerComponent* Pointer = nullptr;
	switch (InHand)
	{
		case EControllerHand::Left: Pointer = LeftControllerPointer; break;
		case EControllerHand::Right: Pointer = RightControllerPointer; break;
	}

	if (Pointer && Pointer->IsEnabled())
	{
		const FHitResult& LaserHitResult = Pointer->GetHitResult();
		OutLaserStart = Pointer->GetComponentLocation();
		OutLaserEnd = LaserHitResult.bBlockingHit
			? LaserHitResult.ImpactPoint
			: LaserHitResult.TraceEnd;

		return true;
	}

	return false;
}


void AXRCreativeAvatar::SetComponentTickInEditor(UActorComponent* InComponent, bool bInShouldTickInEditor)
{
	if (!InComponent || InComponent->GetOwner() != this)
	{
		return;
	}

	InComponent->bTickInEditor = bInShouldTickInEditor;
}


void AXRCreativeAvatar::RegisterObjectForInput(UObject* InObject)
{
	if (IsValid(InputComponent) && IsValid(InObject))
	{
		InputComponent->ClearBindingsForObject(InObject);
		UInputDelegateBinding::BindInputDelegates(InObject->GetClass(), InputComponent, InObject);
	}
}


void AXRCreativeAvatar::UnregisterObjectForInput(UObject* InObject)
{
	if (IsValid(InputComponent) && InObject)
	{
		InputComponent->ClearBindingsForObject(InObject);
	}
}


void AXRCreativeAvatar::AddInputMappingContext(UInputMappingContext* InContext, int32 InPriority,const FModifyContextOptions Options)
{
	if (IEnhancedInputSubsystemInterface* EnhancedInputSubsystemInterface = GetEnhancedInputSubsystemInterface())
	{
		if (IsValid(InContext))
		{
			EnhancedInputSubsystemInterface->AddMappingContext(InContext, InPriority, Options);
		}
	}
}


void AXRCreativeAvatar::RemoveInputMappingContext(UInputMappingContext* InContext, const FModifyContextOptions Options)
{
	if (IEnhancedInputSubsystemInterface* EnhancedInputSubsystemInterface = GetEnhancedInputSubsystemInterface())
	{
		if (IsValid(InContext))
		{
			EnhancedInputSubsystemInterface->RemoveMappingContext(InContext, Options);
		}
	}
}

void AXRCreativeAvatar::ClearAllInputMappings()
{
	if (IEnhancedInputSubsystemInterface* EnhancedInputSubsystemInterface = GetEnhancedInputSubsystemInterface())
	{
		EnhancedInputSubsystemInterface->ClearAllMappings();
	}
}

//** Cues up a Haptic effect to be processed on tick in ProcessHaptics() //
void AXRCreativeAvatar::PlayHapticEffect(UHapticFeedbackEffect_Base* HapticEffect, const int ControllerID, const EControllerHand Hand, float Scale, bool bLoop)
{
	if (HapticEffect)
	{
		switch (Hand)
		{
		case EControllerHand::Left:
			ActiveHapticEffect_Left.Reset();
			ActiveHapticEffect_Left = MakeShareable(new FActiveHapticFeedbackEffect (HapticEffect, Scale, bLoop));
			break;
		case EControllerHand::Right:
			ActiveHapticEffect_Right.Reset();
			ActiveHapticEffect_Right = MakeShareable(new FActiveHapticFeedbackEffect(HapticEffect, Scale, bLoop));
			break;
		default:
			UE_LOGF(LogXRCreative, Warning, "invalid hand specified (%d) for feedback effect %ls. Only left and right hand controllers supported", (int32)Hand, *HapticEffect->GetName());
		}
	}
}

void AXRCreativeAvatar::StopHapticEffect(EControllerHand Hand, const int ControllerID)
{
	if (Hand == EControllerHand::Left)
	{
		ActiveHapticEffect_Left.Reset();
	}
	else if (Hand == EControllerHand::Right)
	{
		ActiveHapticEffect_Right.Reset();
	}

	IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
	if (InputInterface)
	{
		FHapticFeedbackValues Values( 0.f,  0.f);
		InputInterface->SetHapticFeedbackValues(ControllerID, (int32)Hand, Values);
	}
}


//**TO DO need to add check to see if Game is paused here as happens in PlayerController//

void AXRCreativeAvatar::ProcessHaptics(const float DeltaTime)
{
	FHapticFeedbackValues LeftHaptics, RightHaptics;
	bool bLeftHapticsNeedUpdate = false;
	bool bRightHapticsNeedUpdate = false;
	
	if (ActiveHapticEffect_Left.IsValid())
	{
		const bool bPlaying = ActiveHapticEffect_Left->Update(DeltaTime, LeftHaptics);
		if (!bPlaying)
		{
			ActiveHapticEffect_Left->bLoop ? ActiveHapticEffect_Left->Restart() : ActiveHapticEffect_Left.Reset();
		}
		bLeftHapticsNeedUpdate = true;
	}

	if (ActiveHapticEffect_Right.IsValid())
	{
		const bool bPlaying = ActiveHapticEffect_Right->Update(DeltaTime, RightHaptics);
		if (!bPlaying)
		{
			ActiveHapticEffect_Right->bLoop ? ActiveHapticEffect_Right->Restart() : ActiveHapticEffect_Right.Reset();
		}
		bRightHapticsNeedUpdate = true;
	}

	if (FSlateApplication::IsInitialized())
	{
		int32 ControllerID = 0;

		IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
		if (InputInterface)
		{
			if (bLeftHapticsNeedUpdate)
			{
				InputInterface->SetHapticFeedbackValues(ControllerID, (int32) EControllerHand::Left, LeftHaptics);
			}
			if (bRightHapticsNeedUpdate)
			{
				InputInterface->SetHapticFeedbackValues(ControllerID, (int32) EControllerHand::Right, RightHaptics);
			}
		}
	}
}

IEnhancedInputSubsystemInterface* AXRCreativeAvatar::GetEnhancedInputSubsystemInterface() const
{
	if (const UWorld* World = GetWorld(); IsValid(World) && World->IsGameWorld())
	{
		if (const ULocalPlayer* FirstLocalPlayer = World->GetFirstLocalPlayerFromController())
		{
			return ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(FirstLocalPlayer);
		}
	}
#if WITH_EDITOR
	else if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UEnhancedInputEditorSubsystem>();
	}
#endif
	return nullptr;
}


void AXRCreativeAvatar::RegisterInputComponent()
{
	// Ensure we start from a clean slate
	UnregisterInputComponent();
	
	if (const UWorld* World = GetWorld(); IsValid(World) && World->IsGameWorld())
	{
		if (APlayerController* PC = World->GetFirstPlayerController())
		{
			PC->PushInputComponent(InputComponent);
			bIsInputRegistered = true;
		}
	}
#if WITH_EDITOR
	else if (GEditor)
	{
		if (UEnhancedInputEditorSubsystem* EditorInputSubsystem = GEditor->GetEditorSubsystem<UEnhancedInputEditorSubsystem>())
		{
			EditorInputSubsystem->PushInputComponent(InputComponent);
			EditorInputSubsystem->StartConsumingInput();
			bIsInputRegistered = true;
		}
	}
#endif
}


void AXRCreativeAvatar::UnregisterInputComponent()
{
	// Removes the component from both editor and runtime input systems if possible
	if (const UWorld* World = GetWorld(); IsValid(World) && World->IsGameWorld())
	{
		if (APlayerController* PC = World->GetFirstPlayerController())
		{
			PC->PopInputComponent(InputComponent);
		}
	}
#if WITH_EDITOR
	if (GEditor)
	{
		if (UEnhancedInputEditorSubsystem* EditorInputSubsystem = GEditor->GetEditorSubsystem<UEnhancedInputEditorSubsystem>())
		{
			EditorInputSubsystem->PopInputComponent(InputComponent);
		}
	}
#endif

	bIsInputRegistered = false;
}


AActor* AXRCreativeAvatar::InternalSpawnTransientActor(
	TSubclassOf<AActor> ActorClass,
	const FString& ActorName,
	TOptional<TFunctionRef<void (AActor*)>> DeferredConstructionCallback)
{
#if 0 // TODO: Move defer support into UEditorWorldExtension if we want world transitions
	if (IVREditorModule::IsAvailable())
	{
		UVREditorModeBase* VrMode = IVREditorModule::Get().GetVRModeBase();
		if (VrMode && (VrMode->GetWorld() == this->GetWorld()))
		{
			return VrMode->SpawnTransientSceneActor(ActorClass, ActorName);
		}
	}
#endif

	UWorld* World = GetWorld();

#if WITH_EDITOR
	const bool bWasWorldPackageDirty = World->GetOutermost()->IsDirty();
#endif

	FActorSpawnParameters ActorSpawnParameters;
	ActorSpawnParameters.Name = MakeUniqueObjectName(World->GetCurrentLevel(), ActorClass, *ActorName);
	ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	ActorSpawnParameters.ObjectFlags = RF_Transient | RF_DuplicateTransient;

	if (DeferredConstructionCallback.IsSet())
	{
		// NB: User is responsible for calling FinishSpawning, either
		// during callback or later on returned actor
		ActorSpawnParameters.bDeferConstruction = true;
	}

	AActor* Actor = World->SpawnActor<AActor>(ActorClass, ActorSpawnParameters);

	if (DeferredConstructionCallback.IsSet())
	{
		(*DeferredConstructionCallback)(Actor);
	}

#if WITH_EDITOR
	// Don't dirty the level file after spawning a transient actor
	if (!bWasWorldPackageDirty)
	{
		World->GetOutermost()->SetDirtyFlag(false);
	}
#endif

	return Actor;
}


ALevelSequenceActor* AXRCreativeAvatar::OpenLevelSequence(ULevelSequence* LevelSequence)
{
	if (!ensure(LevelSequence))
	{
		return nullptr;
	}

	ALevelSequenceActor* Actor = CastChecked<ALevelSequenceActor>(SpawnTransientActor(
		ALevelSequenceActor::StaticClass(), TEXT("XRCreativeSequence"),
		[LevelSequence](AActor* NewActor)
		{
			ALevelSequenceActor* NewLSA = CastChecked<ALevelSequenceActor>(NewActor);
			NewLSA->SetSequence(LevelSequence);
			NewLSA->InitializePlayer();
			NewLSA->FinishSpawning(FTransform::Identity);
		}));

	return Actor;
}


#if WITH_EDITOR

void AXRCreativeAvatar::MultiUserStartup()
{
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			OnSessionStartupHandle = ConcertClient->OnSessionStartup().AddUObject(this, &AXRCreativeAvatar::HandleSessionStartup);
			OnSessionShutdownHandle = ConcertClient->OnSessionShutdown().AddUObject(this, &AXRCreativeAvatar::HandleSessionShutdown);

			if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
			{
				HandleSessionStartup(ConcertClientSession.ToSharedRef());
			}
		}
	}
}


void AXRCreativeAvatar::MultiUserShutdown()
{
	if (IMultiUserClientModule::IsAvailable())
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IMultiUserClientModule::Get().GetClient())
		{
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();

			if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
			{
				HandleSessionShutdown(ConcertClientSession.ToSharedRef());
			}

			ConcertClient->OnSessionStartup().Remove(OnSessionStartupHandle);
			OnSessionStartupHandle.Reset();

			ConcertClient->OnSessionShutdown().Remove(OnSessionShutdownHandle);
			OnSessionShutdownHandle.Reset();
		}
	}
}


void AXRCreativeAvatar::HandleSessionStartup(TSharedRef<IConcertClientSession> InSession)
{
	WeakSession = InSession;

	//InSession->RegisterCustomEventHandler<FXRCreativeAvatarEvent>(this, &AXRCreativeAvatar::HandleConcertEvent);
}


void AXRCreativeAvatar::HandleSessionShutdown(TSharedRef<IConcertClientSession> InSession)
{
	if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
	{
		//Session->UnregisterCustomEventHandler<FXRCreativeAvatarEvent>(this);
	}

	WeakSession.Reset();
}


#endif // #if WITH_EDITOR
