// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeGameMode.h"
#include "XRCreativeToolset.h"


UXRCreativeToolset* AXRCreativeGameMode::GetToolset() const
{
	return ToolsetClass.LoadSynchronous();
}
