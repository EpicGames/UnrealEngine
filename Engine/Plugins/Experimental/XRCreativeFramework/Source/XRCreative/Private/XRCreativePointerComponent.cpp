// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativePointerComponent.h"
#include "MotionControllerComponent.h"
#include "OneEuroFilter.h"


UXRCreativePointerComponent::UXRCreativePointerComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bTickEvenWhenPaused = true;

	bAutoActivate = true;
	bWantsInitializeComponent = true;

	bEnabled = true;
	TraceMaxLength = 30000.0f;
	SmoothingLag = 0.007f;
	SmoothingMinCutoff = 0.9f;

	SmoothingFilter = MakePimpl<UE::XRCreative::FOneEuroFilter>(SmoothingMinCutoff, SmoothingLag, 1.0f);
}


void UXRCreativePointerComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// Make sure we tick after our owning motion controller.
	USceneComponent* Ancestor = this;
	while (Ancestor->GetAttachParent())
	{
		Ancestor = Ancestor->GetAttachParent();
		if (UMotionControllerComponent* MotionParent = Cast<UMotionControllerComponent>(Ancestor))
		{
			PrimaryComponentTick.AddPrerequisite(MotionParent, MotionParent->PrimaryComponentTick);
		}
	}
}


void UXRCreativePointerComponent::UninitializeComponent()
{
	Super::UninitializeComponent();
}


FCollisionQueryParams UXRCreativePointerComponent::GetQueryParams() const
{
	FCollisionQueryParams QueryParams;
	QueryParams.AddIgnoredActors(IgnoredActors);
	QueryParams.AddIgnoredComponents(IgnoredComponents);
	QueryParams.bTraceComplex = true;
	return QueryParams;
}


void UXRCreativePointerComponent::TickComponent(float InDeltaTime, enum ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction)
{
	Super::TickComponent(InDeltaTime, InTickType, InThisTickFunction);

	RawTraceEnd = GetComponentLocation() + (GetForwardVector() * TraceMaxLength);
	const FVector FilteredTraceEnd = SmoothingFilter->Filter(RawTraceEnd, InDeltaTime);

	GetWorld()->LineTraceSingleByChannel(HitResult, GetComponentLocation(), FilteredTraceEnd, ECC_Visibility, GetQueryParams());
}


FVector UXRCreativePointerComponent::GetRawTraceEnd(const bool bScaledByImpact /* = true*/) const
{
	if (bScaledByImpact)
	{
		const FVector TraceStart = GetComponentLocation();
		return TraceStart + ((RawTraceEnd - TraceStart) * HitResult.Time);
	}
	else
	{
		return RawTraceEnd;
	}

}


FVector UXRCreativePointerComponent::GetFilteredTraceEnd(const bool bScaledByImpact /* = true */) const
{
	if (bScaledByImpact)
	{
		const FVector TraceStart = GetComponentLocation();
		return TraceStart + ((HitResult.TraceEnd - TraceStart) * HitResult.Time);
	}
	else
	{
		return HitResult.TraceEnd;
	}
}
