// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeSettings.h"


namespace UE::XRCreative::Private
{
	static const FName PluginsCategoryName = TEXT("Plugins");
};


FName UXRCreativeSettings::GetCategoryName() const
{
	return UE::XRCreative::Private::PluginsCategoryName;
}

UXRCreativeSettings* UXRCreativeSettings::GetXRCreativeSettings()
{
	return GetMutableDefault<UXRCreativeSettings>();
}

FName UXRCreativeEditorSettings::GetCategoryName() const
{
	return UE::XRCreative::Private::PluginsCategoryName;
}

UXRCreativeEditorSettings* UXRCreativeEditorSettings::GetXRCreativeEditorSettings()
{
	return GetMutableDefault<UXRCreativeEditorSettings>();
}
