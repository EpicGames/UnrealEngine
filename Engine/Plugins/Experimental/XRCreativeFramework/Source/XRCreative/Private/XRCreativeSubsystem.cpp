// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeSubsystem.h"
#include "XRCreativeSettings.h"
#include "Types/MVVMViewModelCollection.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#	include "IVREditorModule.h"
#endif


void UXRCreativeSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	ViewModelCollection = NewObject<UMVVMViewModelCollectionObject>(this);
}


#if WITH_EDITOR
bool UXRCreativeSubsystem::EnterVRMode()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	if (VREditorModule.IsVREditorAvailable())
	{
		VREditorModule.EnableVREditor(true);
		return true;
	}

	return false;
}


void UXRCreativeSubsystem::ExitVRMode()
{
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	if (VREditorModule.IsVREditorEnabled())
	{
		VREditorModule.EnableVREditor(false);
	}
}
#endif // #if WITH_EDITOR
