// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeTeleporter.h"
#include "MotionControllerComponent.h"
#include "Components/StaticMeshComponent.h"


// Sets default values
AXRCreativeTeleporter::AXRCreativeTeleporter(const FObjectInitializer& ObjectInitializer)
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;

	USceneComponent* SceneRoot = CreateDefaultSubobject<USceneComponent>("Root");
	SetRootComponent(SceneRoot);

	LeftController = CreateDefaultSubobject<UMotionControllerComponent>("LeftController");
	LeftController->SetupAttachment(GetRootComponent());
	LeftController->bTickInEditor = true;
	LeftController->MotionSource = "Left";

	RightController = CreateDefaultSubobject<UMotionControllerComponent>("RightController");
	RightController->SetupAttachment(GetRootComponent());
	RightController->bTickInEditor = true;
	RightController->MotionSource = "Right";

	TeleportMesh = CreateDefaultSubobject<UStaticMeshComponent>("TeleportMesh");
	TeleportMesh->SetupAttachment(GetRootComponent());
	TeleportMesh->SetMobility(EComponentMobility::Movable);

	HeadMountedDisplay = CreateDefaultSubobject<UMotionControllerComponent>("HeadMountedDisplay");
	HeadMountedDisplay->SetupAttachment(GetRootComponent());
	HeadMountedDisplay->bTickInEditor = true;
	HeadMountedDisplay->MotionSource = "HMD";

	LeftControllerVisual = CreateDefaultSubobject<UStaticMeshComponent>("LeftControllerVisual");
	LeftControllerVisual->SetupAttachment(LeftController);

	RightControllerVisual = CreateDefaultSubobject<UStaticMeshComponent>("RightControllerVisual");
	RightControllerVisual->SetupAttachment(RightController);
	
	HMDVisual = CreateDefaultSubobject<UStaticMeshComponent>("HMDVisual");
	HMDVisual->SetupAttachment(HeadMountedDisplay);
	
}

// Called when the game starts or when spawned
void AXRCreativeTeleporter::BeginPlay()
{
	Super::BeginPlay();
	
}

// Called every frame
void AXRCreativeTeleporter::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
}

