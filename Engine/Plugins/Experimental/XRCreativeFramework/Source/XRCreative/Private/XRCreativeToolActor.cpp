// Copyright Epic Games, Inc. All Rights Reserved.


#include "XRCreativeToolActor.h"


// Sets default values
AXRCreativeToolActor::AXRCreativeToolActor()
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;
}

// Called when the game starts or when spawned
void AXRCreativeToolActor::BeginPlay()
{
	Super::BeginPlay();
	
}

// Called every frame
void AXRCreativeToolActor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
}

