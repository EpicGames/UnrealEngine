// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeToolset.h"

#include "InputMappingContext.h"
#include "XRCreativeLog.h"
#include "Logging/LogMacros.h"

#if WITH_EDITOR
#	include "XRCreativeSettings.h"
#endif

DEFINE_LOG_CATEGORY(LogXRCreativeToolset);

UInputMappingContext* UXRCreativeBlueprintableTool::GetToolInputMappingContext()
{
	UInputMappingContext* InputMappingContext = RightHandedInputMappingContext;
	
	#if WITH_EDITOR
		UXRCreativeEditorSettings* Settings = UXRCreativeEditorSettings::GetXRCreativeEditorSettings();

		if (Settings->Handedness == EXRCreativeHandedness::Left && LeftHandedInputMappingContext)
		{
			InputMappingContext = LeftHandedInputMappingContext;
		}
		else if (Settings->Handedness == EXRCreativeHandedness::Left && !LeftHandedInputMappingContext)
		{
			InputMappingContext = RightHandedInputMappingContext;
			UE_LOGF(LogXRCreativeToolset, Warning, "Handedness is Left but no Left Handed Input Mapping Context found in Toolset - Using RightHand.");
		}
	
	#endif
	
	return InputMappingContext; 
}


