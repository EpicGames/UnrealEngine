// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Pawn.h"
#include "XRCreativeAvatar.generated.h"



#if WITH_EDITOR
class IConcertClientSession;
#endif

struct FActiveHapticFeedbackEffect;
class IEnhancedInputSubsystemInterface;
class UHapticFeedbackEffect_Base;
class UInputComponent;
class UInputMappingContext;
class UMotionControllerComponent;
class USkeletalMeshComponent;
class UWidgetComponent;
class UWidgetInteractionComponent;
class UXRCreativeITFComponent;
class UXRCreativePointerComponent;
class UXRCreativeTool;
class UXRCreativeToolset;

class ALevelSequenceActor;
struct FMovieSceneSequencePlaybackSettings;
class ULevelSequence;


UCLASS()
class XRCREATIVE_API AXRCreativeAvatar : public APawn
{
	GENERATED_BODY()

public:
	AXRCreativeAvatar(const FObjectInitializer& ObjectInitializer);

	virtual bool HasLocalNetOwner() const override
	{
		// FIXME?: UMotionControllerComponent workaround
		return true;
	}

	virtual void OnConstruction(const FTransform& InTransform) override;
	virtual void BeginDestroy() override;
	virtual void Tick(float InDeltaSeconds) override;
	virtual void BeginPlay() override;
	virtual void GetActorEyesViewPoint(FVector& Location, FRotator& Rotation) const override;

	void ConfigureToolset(UXRCreativeToolset* InToolset);
	const UXRCreativeToolset* GetToolset() const { return Toolset; }
	const TArray<TObjectPtr<UXRCreativeTool>>& GetTools() const { return Tools; }

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	FTransform GetHeadTransform() const;

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	FTransform GetHeadTransformRoomSpace() const;

	bool GetLaserForHand(EControllerHand InHand, FVector& OutLaserStart, FVector& OutLaserEnd) const;

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void SetComponentTickInEditor(UActorComponent* Component, bool bShouldTickInEditor);

	/* Registers the given object with the Avatar's Input Component
	 * This allows dynamic input bindings such as input events in blueprints to work correctly
	 * Note: Ensure you call UnregisterObjectForInput when you are finished with the object
	 * otherwise input events will still fire until GC actually destroys the object
	 *
	 * @param Object The object to register
	 */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void RegisterObjectForInput(UObject* Object);

	/* Unregisters the given object with the Avatar's Input Component
	 *
	 * @param Object The object to unregister
	 */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void UnregisterObjectForInput(UObject* Object);

	// Adds an explicitly provided Input Mapping Context to the input system
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void AddInputMappingContext(UInputMappingContext* Context, int32 Priority, const FModifyContextOptions Options);

	// Removes an explicitly provided Input Mapping Context to the input system
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void RemoveInputMappingContext(UInputMappingContext* Context, const FModifyContextOptions Options);
	
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void ClearAllInputMappings();
	/**
	* Called when In-Editor VR mode is started. In-Editor equivalent to Begin Play.
	*/
	
	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category="XR Creative", meta=(DisplayName = "On Enter VR"))
	void BP_OnVRInitialize();

	/** Play haptic feedback asset on a given hand - only left and right supported
	 * @param HapticEffect			The haptic effect to play
	 * @param Hand					Which hand to play the haptic effect on
	 * @param ControllerID			ID of PlayerController if in PIE or runtime (not required in-editor)
	 * @param Scale					Scale between 0.0 and 1.0 on the intensity of playback 
	 */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual void PlayHapticEffect(UHapticFeedbackEffect_Base* HapticEffect, const int ControllerID, const EControllerHand Hand, float Scale, bool bLoop);

	/** Instantly stop a haptic feedback for a given hand - only left and right supported
	 * @param Hand					Which hand to stop the haptic effect on
	 * @param ControllerID			ID of PlayerController if in PIE or runtime (not required in-editor)
	 */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void StopHapticEffect(EControllerHand Hand, const int ControllerID);


	/** Includes special handling to not dirty editor worlds. */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	AActor* SpawnTransientActor(TSubclassOf<AActor> ActorClass, const FString& ActorName)
	{
		return InternalSpawnTransientActor(ActorClass, ActorName, TOptional<TFunctionRef<void (AActor*)>>());
	}

	AActor* SpawnTransientActor(TSubclassOf<AActor> ActorClass, const FString& ActorName, TFunctionRef<void (AActor*)> DeferredConstructionCallback)
	{
		return InternalSpawnTransientActor(ActorClass, ActorName, DeferredConstructionCallback);
	}


	UFUNCTION(BlueprintCallable, Category="XR Creative")
	ALevelSequenceActor* OpenLevelSequence(ULevelSequence* LevelSequence);

protected:
	/*
	 * Gets the input interface for the currently active input method.
	 * Will switch between editor and player based input as needed
	 */
	IEnhancedInputSubsystemInterface* GetEnhancedInputSubsystemInterface() const;

	// Utility functions for registering and unregistering our input component with the correct input system
	void RegisterInputComponent();
	void UnregisterInputComponent();

	bool bIsInputRegistered = false;

	void ProcessHaptics(const float DeltaTime);

	/** Currently playing haptic effects for both the left and right hand */
	TSharedPtr<FActiveHapticFeedbackEffect> ActiveHapticEffect_Left;
	TSharedPtr<FActiveHapticFeedbackEffect> ActiveHapticEffect_Right;

	/** Includes special handling to not dirty editor worlds. */
	AActor* InternalSpawnTransientActor(TSubclassOf<AActor> ActorClass, const FString& ActorName, TOptional<TFunctionRef<void (AActor*)>> DeferredConstructionCallback);

#if WITH_EDITOR
	void MultiUserStartup();
	void MultiUserShutdown();

	void HandleSessionStartup(TSharedRef<IConcertClientSession> InSession);
	void HandleSessionShutdown(TSharedRef<IConcertClientSession> InSession);

	/** Delegate handle for a the callback when a session starts up */
	FDelegateHandle OnSessionStartupHandle;

	/** Delegate handle for a the callback when a session shuts down */
	FDelegateHandle OnSessionShutdownHandle;

	/** Weak pointer to the client session with which to send events. May be null or stale. */
	TWeakPtr<IConcertClientSession> WeakSession;
#endif // #if WITH_EDITOR

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UMotionControllerComponent> LeftController;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UMotionControllerComponent> LeftControllerAim;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UXRCreativePointerComponent> LeftControllerPointer;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UMotionControllerComponent> RightController;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UMotionControllerComponent> RightControllerAim;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UXRCreativePointerComponent> RightControllerPointer;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative Avatar")
	TObjectPtr<USkeletalMeshComponent> LeftControllerModel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative Avatar")
	TObjectPtr<USkeletalMeshComponent> RightControllerModel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative Avatar")
	TObjectPtr<UWidgetComponent> MenuWidget;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UWidgetInteractionComponent> WidgetInteraction;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UXRCreativeITFComponent> ToolsComponent;

	UPROPERTY(BlueprintReadOnly, Category="XR Creative Avatar")
	TObjectPtr<UXRCreativeToolset> Toolset;

	UPROPERTY(BlueprintReadOnly, Category="XR Creative Avatar")
	TArray<TObjectPtr<UXRCreativeTool>> Tools;
};
