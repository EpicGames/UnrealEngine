// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/GameModeBase.h"
#include "XRCreativeGameMode.generated.h"


class UXRCreativeToolset;


UCLASS()
class XRCREATIVE_API AXRCreativeGameMode : public AGameModeBase
{
	GENERATED_BODY()

public:
	UXRCreativeToolset* GetToolset() const;

protected:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TSoftObjectPtr<UXRCreativeToolset> ToolsetClass;
};
