// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "Components/StaticMeshComponent.h"
#include "XRCreativeGizmos.generated.h"


class AXRCreativeAvatar;

namespace UE::GizmoUtil
{
	struct FTransformSubGizmoCommonParams;
	struct FTransformSubGizmoSharedState;
}


/** Responsible for instantiating our UXRCreativeGizmo subclass. */
UCLASS()
class UXRCreativeGizmoBuilder : public UCombinedTransformGizmoBuilder
{
	GENERATED_BODY()

public:
	UXRCreativeGizmoBuilder();

	virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};


/** Implements our UpdateHoverFunction, and introduces an analogous UpdateInteractingFunction. */
UCLASS()
class UXRCreativeGizmo : public UCombinedTransformGizmo
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
	virtual void Tick(float DeltaTime) override;

protected:
	using FTransformSubGizmoCommonParams = UE::GizmoUtil::FTransformSubGizmoCommonParams;
	using FTransformSubGizmoSharedState = UE::GizmoUtil::FTransformSubGizmoSharedState;

	virtual UInteractiveGizmo* AddAxisTranslationGizmo(FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState) override;
	virtual UInteractiveGizmo* AddPlaneTranslationGizmo(FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState) override;
	virtual UInteractiveGizmo* AddAxisRotationGizmo(FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState) override;
	virtual UInteractiveGizmo* AddAxisScaleGizmo(FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState) override;
	virtual UInteractiveGizmo* AddPlaneScaleGizmo(FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState) override;
	virtual UInteractiveGizmo* AddUniformScaleGizmo(FTransformSubGizmoCommonParams& Params, FTransformSubGizmoSharedState& SharedState) override;

protected:
	TFunction<void(UPrimitiveComponent*, bool)> UpdateInteractingFunction;
};


class UXRCreativeGizmoMeshComponent;


UCLASS(Blueprintable, NonTransient)
class AXRCreativeCombinedTransformGizmoActor : public ACombinedTransformGizmoActor
{
	GENERATED_BODY()

public:
	AXRCreativeCombinedTransformGizmoActor();
	
	void SetEnabledElements(ETransformGizmoSubElements EnableElements);

	UPROPERTY(BlueprintReadOnly, Category="Gizmo")
	TObjectPtr<USceneComponent> WorldAligned;

	TWeakObjectPtr<UInteractiveGizmoManager> WeakGizmoManager;

	TWeakObjectPtr<AXRCreativeAvatar> OwnerAvatar;
	
	/* Get the XRCreative Avatar that spawned this Gizmo */
	UFUNCTION(BlueprintCallable, Category="XRCreative|Gizmo")
	AXRCreativeAvatar* GetOwnerAvatar() { return OwnerAvatar.Get(); }

public:
	// NOTE: These properties alias ones inherited from ACombinedTransformGizmoActor
	// as a workaround to expose them as EditAnywhere + BlueprintReadWrite without
	// modifying the base class.

	//////////////////////////////////////////////////////////////////////////

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<USceneComponent> XRSceneRoot;

	//
	// Translation Components
	//

	/** X Axis Translation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRTranslateX;

	/** Y Axis Translation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRTranslateY;

	/** Z Axis Translation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRTranslateZ;


	/** YZ Plane Translation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRTranslateYZ;

	/** XZ Plane Translation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRTranslateXZ;

	/** XY Plane Translation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRTranslateXY;

	//
	// Rotation Components
	//

	/** X Axis Rotation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRRotateX;

	/** Y Axis Rotation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRRotateY;

	/** Z Axis Rotation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRRotateZ;

	/** Z Axis Rotation Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRRotationSphere;

	//
	// Scaling Components
	//

	/** Uniform Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRUniformScale;


	/** X Axis Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRAxisScaleX;

	/** Y Axis Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRAxisScaleY;

	/** Z Axis Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRAxisScaleZ;


	/** YZ Plane Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRPlaneScaleYZ;

	/** XZ Plane Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRPlaneScaleXZ;

	/** XY Plane Scale Component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	TObjectPtr<UXRCreativeGizmoMeshComponent> XRPlaneScaleXY;	
};


DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FXRCreativeGizmoStateChanged, UXRCreativeGizmoMeshComponent*, Component, bool, bNewState);


UCLASS()
class UXRCreativeGizmoMeshComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	void Initialize();
	void UpdateHoverState(bool bInHovering);
	void UpdateInteractingState(bool bInInteracting);

	UPROPERTY(BlueprintAssignable)
	FXRCreativeGizmoStateChanged OnHoveringChanged;

	UPROPERTY(BlueprintAssignable)
	FXRCreativeGizmoStateChanged OnInteractingChanged;

	FTransform CalcViewDependent(const FViewCameraState& InView, EToolContextCoordinateSystem InCoords) const;

	IGizmoAxisSource* AxisSource = nullptr;

protected:
	UPROPERTY(BlueprintReadOnly, Category="Gizmo")
	bool bHovering;

	UPROPERTY(BlueprintReadOnly, Category="Gizmo")
	bool bInteracting;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	float HideAbsoluteViewDotThreshold = -1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	bool bReflectOnPrimaryAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gizmo")
	bool bReflectOnTangentAxes;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInstanceDynamic>> Materials;
};
