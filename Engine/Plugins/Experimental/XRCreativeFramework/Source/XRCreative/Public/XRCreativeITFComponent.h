// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "InputState.h"
#include "ToolContextInterfaces.h"
#include "XRCreativeITFComponent.generated.h"


class AXRCreativeBaseTransformGizmoActor;
class AXRCreativeCombinedTransformGizmoActor;
class FXRCreativeToolsContextTransactionImpl;
class FXRCreativeToolsContextQueriesImpl;
class UInteractiveToolsContext;
class UTypedElementSelectionSet;
class UXRCreativeSelectionInteraction;
class UXRCreativeTransformInteraction;
class UXRCreativeITFRenderComponent;
class UXRCreativePointerComponent;

#if WITH_EDITOR
struct FTransactionContext;
#endif


DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FCanSelectActorPredicate, AActor*, SelectionCandidate);


UCLASS()
class XRCREATIVE_API UXRCreativeITFComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UXRCreativeITFComponent();

	void SetPointerComponent(UXRCreativePointerComponent* InPointer);

	virtual void InitializeComponent() override;
	virtual void UninitializeComponent() override;
	virtual void TickComponent(float InDeltaTime, enum ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction) override;

	bool IsInEditor() const;


	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	bool CanUndo() const;

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	bool CanRedo() const;

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void Undo();

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void Redo();

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnUndoRedo);

	UPROPERTY(BlueprintAssignable, Category="XR Creative|Tools")
	FOnUndoRedo OnUndo;

	UPROPERTY(BlueprintAssignable, Category="XR Creative|Tools")
	FOnUndoRedo OnRedo;


	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void LeftMousePressed();

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void LeftMouseReleased();


	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	UTypedElementSelectionSet* GetSelectionSet() const;


	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	bool HaveActiveTool();


	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	EToolContextCoordinateSystem GetCurrentCoordinateSystem() const { return CurrentCoordinateSystem; }

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void SetCurrentCoordinateSystem(EToolContextCoordinateSystem CoordSystem);


	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	EToolContextTransformGizmoMode GetCurrentTransformGizmoMode() const { return CurrentTransformGizmoMode; }

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void SetCurrentTransformGizmoMode(EToolContextTransformGizmoMode GizmoMode);
	
	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	void RecreateActorPhysicsState(AActor* Actor);

	UPROPERTY()
	TWeakObjectPtr<AXRCreativeCombinedTransformGizmoActor> GizmoActor;

	UFUNCTION(BlueprintCallable, Category="XR Creative|Tools")
	AXRCreativeCombinedTransformGizmoActor* GetGizmoActor() const { return GizmoActor.Get(); }
	
	TSubclassOf<AXRCreativeCombinedTransformGizmoActor> GetCombinedGizmoActorClass() const { return CombinedGizmoActorClass; }

protected:
	UPROPERTY(EditAnywhere, Category="XR Creative")
	TSubclassOf<AXRCreativeCombinedTransformGizmoActor> CombinedGizmoActorClass;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative")
	TArray<TSubclassOf<AActor>> UnselectableActorClasses;

	UPROPERTY(BlueprintReadWrite, Category="XR Creative|Tools")
	FCanSelectActorPredicate CanSelectPredicate;

	UPROPERTY()
	TObjectPtr<UXRCreativePointerComponent> PointerComponent;

	UPROPERTY()
	TObjectPtr<UInteractiveToolsContext> ToolsContext;

	UPROPERTY()
	TObjectPtr<UTypedElementSelectionSet> SelectionSet;

	UPROPERTY()
	TObjectPtr<UXRCreativeSelectionInteraction> SelectionInteraction;

	UPROPERTY()
	TObjectPtr<UXRCreativeTransformInteraction> TransformInteraction;

	UPROPERTY()
	TObjectPtr<UXRCreativeITFRenderComponent> PDIRenderComponent;

	UPROPERTY()
	EToolContextCoordinateSystem CurrentCoordinateSystem = EToolContextCoordinateSystem::World;

	UPROPERTY()
	EToolContextTransformGizmoMode CurrentTransformGizmoMode = EToolContextTransformGizmoMode::Combined;

protected:
	void ToolsTick(float InDeltaTime);

#if WITH_EDITOR
	void EditorToolsTick(float InDeltaTime);

	void HandleTransactorUndo(const FTransactionContext& TransactionContext, bool Succeeded);
	void HandleTransactorRedo(const FTransactionContext& TransactionContext, bool Succeeded);
#endif

protected:
	bool bIsShuttingDown = false;

	TSharedPtr<FXRCreativeToolsContextQueriesImpl> ContextQueriesAPI;
	TSharedPtr<FXRCreativeToolsContextTransactionImpl> ContextTransactionsAPI;

	FVector2D PrevMousePosition = FVector2D::ZeroVector;
	FInputDeviceState CurrentMouseState;
	bool bPendingMouseStateChange = false;

	FViewCameraState CurrentViewCameraState;
};
