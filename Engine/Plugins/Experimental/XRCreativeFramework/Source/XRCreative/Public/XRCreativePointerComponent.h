// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollisionQueryParams.h"
#include "Components/SceneComponent.h"
#include "Engine/HitResult.h"
#include "XRCreativePointerComponent.generated.h"


namespace UE::XRCreative
{
	class FOneEuroFilter;
}


UCLASS(ClassGroup = ("XR Creative"), meta=(BlueprintSpawnableComponent))
class XRCREATIVE_API UXRCreativePointerComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	UXRCreativePointerComponent();

	virtual void InitializeComponent() override;
	virtual void UninitializeComponent() override;
	virtual void TickComponent(float InDeltaTime, enum ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction) override;

	/**
	* If bScaledByImpact is false, this returns the raw input to the smoothing filter, `TraceMaxLength` units away in the +X direction.
	* If bScaledByImpact is true, the magnitude is shortened to the length of the blocking hit (if any).
	*/
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	FVector GetRawTraceEnd(const bool bScaledByImpact = true) const;

	/**
	* If bScaledByImpact is false, this is the smoothed filter output, roughly `TraceMaxLength` units away, roughly in the +X direction.
	* If bScaledByImpact is true, the magnitude is shortened to match the length of a blocking hit (if any).
	*/
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	FVector GetFilteredTraceEnd(const bool bScaledByImpact = true) const;

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	const FHitResult& GetHitResult() const { return HitResult; }

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	bool IsEnabled() const { return bEnabled; }

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void SetEnabled(bool bInEnabled) { bEnabled = bInEnabled; }

	FCollisionQueryParams GetQueryParams() const;

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative")
	float TraceMaxLength;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative")
	float SmoothingLag;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative")
	float SmoothingMinCutoff;

	UPROPERTY(EditAnywhere, Getter=IsEnabled, Setter=SetEnabled, Category="XR Creative")
	bool bEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative")
	TArray<TObjectPtr<AActor>> IgnoredActors;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="XR Creative")
	TArray<TObjectPtr<UPrimitiveComponent>> IgnoredComponents;

protected:
	FVector RawTraceEnd;
	FHitResult HitResult;

	TPimplPtr<UE::XRCreative::FOneEuroFilter> SmoothingFilter;
};
