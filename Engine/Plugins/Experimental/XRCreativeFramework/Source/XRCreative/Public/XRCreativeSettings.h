// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "XRCreativeSettings.generated.h"


UENUM(BlueprintType)
enum class EXRCreativeHandedness : uint8
{
	Left	UMETA(DisplayName = "Left"),
	Right	UMETA(DisplayName = "Right"),
};


/**
 * Per project settings for XRCreative.
 */
UCLASS(Config=XRCreativeSettings, DefaultConfig, DisplayName="XR Creative")
class XRCREATIVE_API UXRCreativeSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const override;
	//~ End UDeveloperSettings interface

	UFUNCTION(BlueprintPure, Category="XR Creative")
	static UXRCreativeSettings* GetXRCreativeSettings();
};


/**
 * Per user settings for XRCreative Editor.
 */
UCLASS(Config=EditorPerProjectUserSettings, meta=(DisplayName="XR Creative Editor"))

class XRCREATIVE_API UXRCreativeEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const override;
	//~ End UDeveloperSettings interface

	UFUNCTION(BlueprintPure, Category="XR Creative Editor")
	static UXRCreativeEditorSettings* GetXRCreativeEditorSettings();

	/** Manages Left/Right handedness user preferences.
	 * Modifying this setting requires an editor restart to take effect.
	 **/
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="XR Creative", meta=(DisplayName="Handedness"))
	EXRCreativeHandedness Handedness = EXRCreativeHandedness::Right;
};
