// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "XRCreativeSubsystem.generated.h"


class UMVVMViewModelCollectionObject;


UCLASS()
class XRCREATIVE_API UXRCreativeSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	UFUNCTION(BlueprintCallable, Category="XR Creative|Viewmodel")
	UMVVMViewModelCollectionObject* GetViewModelCollection() const
	{
		return ViewModelCollection;
	}

#if WITH_EDITOR
	/** Enter VR Mode */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	static bool EnterVRMode();

	/** Exit VR Mode */
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	static void ExitVRMode();
#endif // #if WITH_EDITOR

private:
	UPROPERTY(Transient)
	TObjectPtr<UMVVMViewModelCollectionObject> ViewModelCollection;
};
