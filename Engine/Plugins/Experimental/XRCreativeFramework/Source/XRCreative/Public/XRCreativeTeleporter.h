// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "XRCreativeTeleporter.generated.h"

class UMotionControllerComponent;
class UStaticMeshComponent;


UCLASS()
class XRCREATIVE_API AXRCreativeTeleporter : public AActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	AXRCreativeTeleporter(const FObjectInitializer& ObjectInitializer);

	virtual bool HasLocalNetOwner() const override
	{
		//copied from Avatar code TODO: Fixme?
		return true;
	}

	// Called every frame
	virtual void Tick(float DeltaTime) override;

protected:
	// Called when the game starts or when spawned
	virtual void BeginPlay() override;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UMotionControllerComponent> LeftController;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UMotionControllerComponent> RightController;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UStaticMeshComponent> TeleportMesh;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UMotionControllerComponent> HeadMountedDisplay;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UStaticMeshComponent> LeftControllerVisual;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UStaticMeshComponent> RightControllerVisual;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="XR Creative Teleporter")
	TObjectPtr<UStaticMeshComponent> HMDVisual;
		
};
