// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "XRCreativeToolActor.generated.h"

class AXRCreativeAvatar;

UCLASS(Blueprintable, Abstract)
class XRCREATIVE_API AXRCreativeToolActor : public AActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	AXRCreativeToolActor();

	UPROPERTY(BlueprintReadWrite, Category="XR Creative")
	TObjectPtr<AXRCreativeAvatar> OwnerAvatar;

protected:
	// Called when the game starts or when spawned
	virtual void BeginPlay() override;

public:
	// Called every frame
	virtual void Tick(float DeltaTime) override;

	//Called manually by the Avatar that spawns them. 

	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable,Category="XR Creative")
	void InitializeTool();

	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category="XR Creative")
	void TickTool(float DeltaSeconds);
	
	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category="XR Creative")
	bool ShutDownTool();
};
