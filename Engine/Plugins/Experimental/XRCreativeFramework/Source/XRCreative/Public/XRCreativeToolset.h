// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonActivatableWidget.h"
#include "Engine/DataAsset.h"

#include "XRCreativeToolset.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogXRCreativeToolset, Log, All);

class UMVVMViewModelBase;
class UCommonButtonStyle;
class UCommonTextStyle;
class UInputMappingContext;
class AXRCreativeToolActor;


UCLASS()
class UXRCreativePaletteTab : public UCommonActivatableWidget
{
	GENERATED_BODY()
};


UCLASS()
class UXRCreativePalette : public UCommonActivatableWidget
{
	GENERATED_BODY()

public:
	void SetOwner(class AXRCreativeAvatar* InOwner) { Owner = InOwner; }

protected:
	UPROPERTY(BlueprintReadOnly, Category="XR Creative")
	TArray<TObjectPtr<UXRCreativePaletteTab>> Tabs;

	UPROPERTY(BlueprintReadWrite, Category="XR Creative")
	TObjectPtr<AXRCreativeAvatar> Owner;
};


//////////////////////////////////////////////////////////////////////////


UCLASS(Abstract, BlueprintType)
class UXRCreativeTool : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual FName GetToolName() const PURE_VIRTUAL( UXRCreativeTool::GetToolName, return NAME_None; );

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual FText GetDisplayName() const PURE_VIRTUAL( UXRCreativeTool::GetDisplayName, return FText::GetEmpty(); );
	
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual TSubclassOf<class UXRCreativePaletteToolTab> GetPaletteTabClass() const PURE_VIRTUAL( UXRCreativeTool::GetPaletteTabClass, return nullptr; );


};


UCLASS(Abstract, Blueprintable)
class XRCREATIVE_API UXRCreativeBlueprintableTool : public UXRCreativeTool
{
	GENERATED_BODY()

public:
	virtual FName GetToolName() const override { return ToolName; }
	virtual FText GetDisplayName() const override { return DisplayName; }
	virtual TSubclassOf<UXRCreativePaletteToolTab> GetPaletteTabClass() const override { return PaletteTabClass; }

	UFUNCTION(BlueprintPure, Category="XR Creative")
	UInputMappingContext* GetToolInputMappingContext();

protected:
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category="XR Creative")
	FName ToolName;

	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category="XR Creative")
	FText DisplayName;

	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category="XR Creative")
	TSubclassOf<UMVVMViewModelBase> ToolViewmodel;

	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category="XR Creative")
	TSubclassOf<UXRCreativePaletteToolTab> PaletteTabClass;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="XR Creative")
	TSubclassOf<AXRCreativeToolActor> ToolActor;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="XR Creative")
	TObjectPtr<UInputMappingContext> RightHandedInputMappingContext;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="XR Creative")
	TObjectPtr<UInputMappingContext> LeftHandedInputMappingContext;

	
};


UCLASS()
class UXRCreativePaletteToolTab : public UXRCreativePaletteTab
{
	GENERATED_BODY()

protected:
	UPROPERTY(BlueprintReadOnly, Category="XR Creative")
	TWeakObjectPtr<UXRCreativeTool> Tool;
};


//////////////////////////////////////////////////////////////////////////

USTRUCT(BlueprintType)
struct FXRCreativeToolEntry
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TSubclassOf<UXRCreativeTool> ToolClass;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TObjectPtr<UTexture2D> ToolIcon;
};


UCLASS(BlueprintType)
class XRCREATIVE_API UXRCreativeToolset : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	// UXRCreativeToolset();
	//
	// ~UXRCreativeToolset();

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	FText ToolsetName;
	
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TSubclassOf<AXRCreativeAvatar> Avatar;

	/** Default Input Mapping is used for Right-Handed users, or if no LeftInputMappingContext is provided. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TObjectPtr<UInputMappingContext> RightHandedInputMappingContext;
	
	/** If Handedness is selected in XRCreative Settings, uses this entry in place of Default/Right  */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TObjectPtr<UInputMappingContext> LeftHandedInputMappingContext;
	
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TArray<FXRCreativeToolEntry> Tools;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative")
	TSubclassOf<UXRCreativePalette> Palette;

	UPROPERTY(EditAnywhere, Category="XR Creative")
	bool bEnableUIMenuActor = false;

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	bool GetEnableUIMenuActor() const { return bEnableUIMenuActor; };

	/*Enable for legacy or custom menu actors.*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="XR Creative", meta=(EditCondition="bEnableUIMenuActor"))
	TSubclassOf<AActor> UIMenuActor;
	
	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual TSubclassOf<AActor> GetUIMenuActor() const { return UIMenuActor; };
};
