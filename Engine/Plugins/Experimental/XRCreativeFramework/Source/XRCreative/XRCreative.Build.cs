// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class XRCreative : ModuleRules
{
	public XRCreative(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CommonUI",
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"EnhancedInput",
				"HeadMountedDisplay",
				"XRBase",
				"InputCore",
				"InteractiveToolsFramework",
				"LevelSequence",
				"ModelViewViewModel",
				"RenderCore", // TODO: Kill this? Only for FlushRenderingCommands
				"Slate",
				"SlateCore",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UMG",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"ConcertSyncClient",
				"ConcertSyncCore",
				"InputEditor",
				"LevelEditor",
				"MultiUserClient",
				"UnrealEd",
				"VREditor",
			});
		}
	}
}
