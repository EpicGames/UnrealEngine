// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeEditorModule.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogXRCreativeEditor);


class FXRCreativeEditorModule : public IModuleInterface
{
private:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};


IMPLEMENT_MODULE(FXRCreativeEditorModule, XRCreativeEditor);
