// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRCreativeEditorUtilityToolActor.h"
#include "GameFramework/InputSettings.h"
#include "Engine/InputDelegateBinding.h"


// Sets default values


AXRCreativeEditorUtilityToolActor::AXRCreativeEditorUtilityToolActor(const FObjectInitializer& ObjectInitializer)
//	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
}

void AXRCreativeEditorUtilityToolActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	CreateEditorInput();
}

void AXRCreativeEditorUtilityToolActor::CreateEditorInput()
{
	if (bReceivesEditorInput && !HasAnyFlags(RF_ClassDefaultObject) && !EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), TEXT("EdUtilActor_InputComponent0"), RF_Transient);
		UInputDelegateBinding::BindInputDelegatesWithSubojects(this, EditorOnlyInputComponent);
	}
}

void AXRCreativeEditorUtilityToolActor::RemoveEditorInput()
{
	ensure(!bReceivesEditorInput);
	
	if (EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent->DestroyComponent();
	}
	EditorOnlyInputComponent = nullptr;
}

#if WITH_EDITOR
void AXRCreativeEditorUtilityToolActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(AXRCreativeEditorUtilityToolActor, bReceivesEditorInput))
	{
		SetReceivesEditorInput(bReceivesEditorInput);
	}
}
#endif


void AXRCreativeEditorUtilityToolActor::SetReceivesEditorInput(bool bInValue)
{
	bReceivesEditorInput = bInValue;
	if (bReceivesEditorInput)
	{
		CreateEditorInput();
	}
	else
	{
		RemoveEditorInput();
	}
}

