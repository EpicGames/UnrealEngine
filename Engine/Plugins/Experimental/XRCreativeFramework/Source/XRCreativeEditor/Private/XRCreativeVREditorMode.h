// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "VREditorModeBase.h"
#include "XRCreativeVREditorMode.generated.h"


class AXRCreativeAvatar;
class IEnhancedInputSubsystemInterface;
class UInputComponent;
class UMotionControllerComponent;
class UXRCreativeToolset;
class UWidgetComponent;
class UWidgetInteractionComponent;


UCLASS(Abstract)
class UXRCreativeVREditorMode : public UVREditorModeBase
{
	GENERATED_BODY()

public:
	UXRCreativeVREditorMode();

	//~ Begin UVREditorModeBase interface
	virtual void Enter() override;
	virtual void Exit(bool bInShouldDisableStereo) override;
	virtual bool WantsToExitMode() const { return bWantsToExitMode; }

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual FTransform GetRoomTransform() const override;

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	virtual FTransform GetHeadTransform() const override;

	virtual bool GetLaserForHand(EControllerHand InHand, FVector& OutLaserStart, FVector& OutLaserEnd) const override;
	//~ End UVREditorModeBase interface

	//~ Begin UEditorWorldExtension interface
	virtual void Tick(float InDeltaSeconds) override;
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	//~ End UEditorWorldExtension interface

	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="On Enter"))
	void BP_OnEnter();

	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="On Exit"))
	void BP_OnExit();

	UFUNCTION(BlueprintImplementableEvent, meta=(DisplayName="Tick"))
	void BP_Tick(float DeltaSeconds);

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void SetRoomTransform(const FTransform& RoomToWorld);

	UFUNCTION(BlueprintCallable, Category="XR Creative")
	void SetHeadTransform(const FTransform& HeadToWorld);

protected:
	//~ Begin UVREditorModeBase interface
	virtual void EnableStereo() override;
	virtual void DisableStereo() override;
	//~ End UVREditorModeBase interface

	bool ValidateSettings();

	void InvalidSettingNotification(const FText& ErrorDetails);

protected:

	UPROPERTY(EditDefaultsOnly, Category="XR Creative")
	TSoftObjectPtr<UXRCreativeToolset> ToolsetClass;

	UPROPERTY(BlueprintReadOnly, Category="XR Creative")
	TObjectPtr<AXRCreativeAvatar> Avatar;

	bool bWantsToExitMode = false;
	bool bFirstTick = false;
};
