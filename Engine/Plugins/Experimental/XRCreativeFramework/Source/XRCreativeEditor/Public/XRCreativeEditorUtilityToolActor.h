// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Components/InputComponent.h"
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Math/Transform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"
#include "CoreMinimal.h"
#include "XRCreativeToolActor.h"
#include "XRCreativeEditorUtilityToolActor.generated.h"



class UInputComponent;
class UObject;
struct FFrame;
struct FPropertyChangedEvent;



UCLASS(Blueprintable, Abstract, meta = (ShowWorldContextPin))
class XRCREATIVEEDITOR_API AXRCreativeEditorUtilityToolActor : public AXRCreativeToolActor
{
	GENERATED_UCLASS_BODY()

	// Standard function to execute
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Editor")
	void Run();

	virtual void OnConstruction(const FTransform& Transform) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Returns the current InputComponent on this utility actor. This will be NULL unless bReceivesEditorInput is set to true. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input|Editor")
	UInputComponent* GetInputComponent() const
	{ 
		return EditorOnlyInputComponent.Get();
	}
	
	UFUNCTION(BlueprintSetter, Category = "Input|Editor")
	void SetReceivesEditorInput(bool bInValue);
	
	UFUNCTION(BlueprintGetter, BlueprintPure, Category = "Input|Editor")
	bool GetReceivesEditorInput() const
	{
		return bReceivesEditorInput;
	}	
	
private:

	/** Creates the EditorOnlyInputComponent if it does not already exist and registers all subobject callbacks to it */
	void CreateEditorInput();

	/** Removes the EditorOnlyInputComponent from this utility actor */
	void RemoveEditorInput();
	
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UInputComponent> EditorOnlyInputComponent;
	
	/** If set to true, then this actor will be able to recieve input delegate callbacks when in the editor. */
	UPROPERTY(EditAnywhere, Category = "Input|Editor", BlueprintSetter = SetReceivesEditorInput, BlueprintGetter = GetReceivesEditorInput)
	bool bReceivesEditorInput = false;
};