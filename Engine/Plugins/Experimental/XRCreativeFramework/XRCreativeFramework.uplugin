{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "XR Creative Framework",
	"Description": "",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "XRCreative",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "XRCreativeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "OpenXR",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
		{
			"Name": "CommonUI",
			"Enabled": true
		},
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "ModelViewViewModel",
			"Enabled":  true
		}
	]
}
